// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ACLPlugin_Android : ACLPlugin
	{
		public ACLPlugin_Android(ReadOnlyTargetRules Target) : base(Target)
		{
			// The Android toolchain does NOT fully support C++20, it lacks support for std::bit_cast
			PrivateDefinitions.Add("UE_RTM_NO_BIT_CAST");
		}
	}
}
