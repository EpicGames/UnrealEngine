{
	"FileVersion": 3,
	"Version": 30100,
	"VersionName": "3.1.0",
	"FriendlyName": "Animation Compression Library",
	"Description": "Use the Animation Compression Library (ACL) to compress AnimSequences.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Modules": [
		{
			"Name": "ACLPlugin",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformDenyList": []
		},
		{
			"Name": "ACLPluginEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformDenyList": []
		}
	]
}
