// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2018 Nicholas Frechette. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ACLPlugin : ModuleRules
	{
		public ACLPlugin(ReadOnlyTargetRules Target) : base(Target)
		{
			string ACLSDKDir = Path.GetFullPath(Path.Combine(ModuleDirectory, "../ThirdParty"));

			PublicSystemIncludePaths.Add(Path.Combine(ACLSDKDir, "acl/includes"));
			PublicSystemIncludePaths.Add(Path.Combine(ACLSDKDir, "acl/external/rtm/includes"));

			PublicDependencyModuleNames.Add("Core");
			PublicDependencyModuleNames.Add("CoreUObject");
			PublicDependencyModuleNames.Add("Engine");
			
			PrivateDependencyModuleNames.Add("TraceLog");

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("DesktopPlatform");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
