// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimBoneCompressionCodec.h"

#include "ACLImpl.h"

THIRD_PARTY_INCLUDES_START
#if WITH_EDITORONLY_DATA
#include <acl/compression/compression_settings.h>
#include <acl/compression/track_array.h>
#include <acl/core/compressed_database.h>
#include <acl/core/iallocator.h>
#endif

#include <acl/core/compressed_tracks.h>
THIRD_PARTY_INCLUDES_END

#include "AnimBoneCompressionCodec_ACLBase.generated.h"

struct FACLCompressedAnimDataBase : public ICompressedAnimData
{
	/** Whether or not compression failed */
	bool bCompressionFailed = false;

	// ICompressedAnimData implementation
	virtual void SerializeCompressedData(UObject* DataOwner, FArchive& Ar) override;
};

struct FACLCompressedAnimData final : public FACLCompressedAnimDataBase
{
	/** Holds the compressed_tracks instance */
	TArrayView<uint8> CompressedByteStream;

	const acl::compressed_tracks* GetCompressedTracks() const { return acl::make_compressed_tracks(CompressedByteStream.GetData()); }

	// ICompressedAnimData implementation
	virtual void Bind(const TArrayView<uint8> BulkData) override { CompressedByteStream = BulkData; }
	virtual int64 GetApproxCompressedSize() const override { return CompressedByteStream.Num(); }
	virtual bool IsValid() const override;
};

/** The base codec implementation for ACL support. */
UCLASS(abstract, MinimalAPI)
class UAnimBoneCompressionCodec_ACLBase : public UAnimBoneCompressionCodec
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** The compression level to use. Higher levels will be slower to compress but yield a lower memory footprint. */
	UPROPERTY(EditAnywhere, Category = "ACL Options")
	TEnumAsByte<ACLCompressionLevel> CompressionLevel;

	/** How to treat phantom tracks. Phantom tracks are not mapped to a skeleton bone. */
	UPROPERTY(EditAnywhere, Category = "ACL Options")
	ACLPhantomTrackMode PhantomTrackMode;

	/** The default virtual vertex distance for normal bones. */
	UPROPERTY(EditAnywhere, Category = "ACL Options", meta = (ClampMin = "0"))
	float DefaultVirtualVertexDistance;

	/** The virtual vertex distance for bones that requires extra accuracy. */
	UPROPERTY(EditAnywhere, Category = "ACL Options", meta = (ClampMin = "0"))
	float SafeVirtualVertexDistance;

	/** The error threshold to use when optimizing and compressing the animation sequence. */
	UPROPERTY(EditAnywhere, Category = "ACL Options", meta = (ClampMin = "0"))
	float ErrorThreshold;

	// UAnimBoneCompressionCodec implementation
	virtual bool IsHighFidelity(const FCompressibleAnimData& CompressibleAnimData) const override;
	virtual bool Compress(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) override;
	virtual void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar) override;
	virtual int64 EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const override;

	// Our implementation
	virtual void PostCompression(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) const {}
	virtual void GetCompressionSettings(const class ITargetPlatform* TargetPlatform, acl::compression_settings& OutSettings) const PURE_VIRTUAL(UAnimBoneCompressionCodec_ACLBase::GetCompressionSettings, );
	virtual TArray<class USkeletalMesh*> GetOptimizationTargets() const { return TArray<class USkeletalMesh*>(); }
#endif

	// UAnimBoneCompressionCodec implementation
	virtual TUniquePtr<ICompressedAnimData> AllocateAnimData() const override;
	virtual void ByteSwapIn(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryReader& MemoryStream) const override;
	virtual void ByteSwapOut(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryWriter& MemoryStream) const override;
};
