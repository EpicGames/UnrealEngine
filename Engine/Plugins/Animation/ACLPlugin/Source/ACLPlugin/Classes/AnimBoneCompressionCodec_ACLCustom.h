// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimBoneCompressionCodec_ACLBase.h"
#include "AnimBoneCompressionCodec_ACLCustom.generated.h"

/** Uses the open source Animation Compression Library with custom settings suitable for debugging purposes. */
UCLASS(MinimalAPI, config = Engine, meta = (DisplayName = "Anim Compress ACL Custom"))
class UAnimBoneCompressionCodec_ACLCustom : public UAnimBoneCompressionCodec_ACLBase
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** The rotation format to use. */
	UPROPERTY(EditAnywhere, Category = Clip)
	TEnumAsByte<ACLRotationFormat> RotationFormat;

	/** The translation format to use. */
	UPROPERTY(EditAnywhere, Category = Clip)
	TEnumAsByte<ACLVectorFormat> TranslationFormat;

	/** The scale format to use. */
	UPROPERTY(EditAnywhere, Category = Clip)
	TEnumAsByte<ACLVectorFormat> ScaleFormat;

	/** The skeletal meshes used to estimate the skinning deformation during compression. */
	UPROPERTY(EditAnywhere, Category = "ACL Options")
	TArray<TObjectPtr<class USkeletalMesh>> OptimizationTargets;

	/** The minimum proportion of keyframes that should be stripped. */
	UPROPERTY(EditAnywhere, Category = "ACL Destructive Options", meta = (ClampMin = "0", ClampMax = "1"))
	FPerPlatformFloat KeyframeStrippingProportion;

	/** The error threshold below which to strip keyframes. If a keyframe can be reconstructed with an error below the threshold, it is stripped. */
	UPROPERTY(EditAnywhere, Category = "ACL Destructive Options", meta = (ClampMin = "0"))
	FPerPlatformFloat KeyframeStrippingThreshold;

	//////////////////////////////////////////////////////////////////////////

	// UAnimBoneCompressionCodec implementation
	virtual void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar) override;

	// UAnimBoneCompressionCodec_ACLBase implementation
	virtual void GetCompressionSettings(const class ITargetPlatform* TargetPlatform, acl::compression_settings& OutSettings) const override;
	virtual TArray<class USkeletalMesh*> GetOptimizationTargets() const override { return OptimizationTargets; }
#endif

	// UAnimBoneCompressionCodec implementation
	virtual void DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs, TArrayView<FTransform>& OutAtoms) const override;
	virtual void DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const override;
};
