// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "ACLImpl.h"

THIRD_PARTY_INCLUDES_START
#include <acl/decompression/database/database.h>
THIRD_PARTY_INCLUDES_END

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimationCompressionLibraryDatabase.h"
#include "AnimBoneCompressionCodec_ACLBase.h"
#include "AnimBoneCompressionCodec_ACLDatabase.generated.h"

struct FACLDatabaseCompressedAnimData final : public FACLCompressedAnimDataBase
{
	/** Maps the compressed_tracks instance. Used in cooked build only. */
	TArrayView<uint8> CompressedByteStream;

	/** Maps the database context instance. Used in cooked build only. */
	acl::database_context<UEDefaultDatabaseSettings>* DatabaseContext = nullptr;

	/** The codec instance that owns us. */
	const class UAnimBoneCompressionCodec_ACLDatabase* Codec = nullptr;

	/** The sequence name hash that owns this data. */
	uint32 SequenceNameHash = 0;

#if WITH_EDITORONLY_DATA
	/** Holds the compressed_tracks instance for the anim sequence */
	TArray<uint8> CompressedClip;
#endif

#if WITH_EDITORONLY_DATA
	const acl::compressed_tracks* GetCompressedTracks() const { return acl::make_compressed_tracks(CompressedClip.GetData()); }
#else
	const acl::compressed_tracks* GetCompressedTracks() const { return acl::make_compressed_tracks(CompressedByteStream.GetData()); }
#endif

	// ICompressedAnimData implementation
#if WITH_EDITORONLY_DATA
	virtual ~FACLDatabaseCompressedAnimData() override;
#endif

	virtual void SerializeCompressedData(UObject* DataOwner, FArchive& Ar) override;
	virtual void Bind(const TArrayView<uint8> BulkData) override;
	virtual int64 GetApproxCompressedSize() const override;
	virtual bool IsValid() const override;
};

/**
  * Uses the open source Animation Compression Library with default settings and database support.
  * The referenced database can be used to strip the least important keyframes on a per platform basis
  * or they can be streamed in/out on demand through Blueprint or C++.
  */
UCLASS(MinimalAPI, config = Engine, meta = (DisplayName = "Anim Compress ACL Database"))
class UAnimBoneCompressionCodec_ACLDatabase : public UAnimBoneCompressionCodec_ACLBase
{
	GENERATED_UCLASS_BODY()

	/** The database asset that will hold the compressed animation data. */
	UPROPERTY(EditAnywhere, Category = "ACL Options")
	TObjectPtr<UAnimationCompressionLibraryDatabase> DatabaseAsset;

#if WITH_EDITORONLY_DATA
	/** The skeletal meshes used to estimate the skinning deformation during compression. */
	UPROPERTY(EditAnywhere, Category = "ACL Options")
	TArray<TObjectPtr<class USkeletalMesh>> OptimizationTargets;

	//////////////////////////////////////////////////////////////////////////
	// UObject
	virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	// UAnimBoneCompressionCodec implementation
	virtual void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar) override;

	// UAnimBoneCompressionCodec_ACLBase implementation
	virtual void PostCompression(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) const override;
	virtual void GetCompressionSettings(const class ITargetPlatform* TargetPlatform, acl::compression_settings& OutSettings) const override;
	virtual TArray<class USkeletalMesh*> GetOptimizationTargets() const override { return OptimizationTargets; }
#endif

	// UAnimBoneCompressionCodec implementation
	virtual TUniquePtr<ICompressedAnimData> AllocateAnimData() const override;
	virtual void ByteSwapIn(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryReader& MemoryStream) const override;
	virtual void ByteSwapOut(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryWriter& MemoryStream) const override;
	virtual void DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs, TArrayView<FTransform>& OutAtoms) const override;
	virtual void DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const override;
};
