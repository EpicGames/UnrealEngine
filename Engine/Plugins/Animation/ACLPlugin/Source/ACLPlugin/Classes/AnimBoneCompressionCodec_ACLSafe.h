// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimBoneCompressionCodec_ACLBase.h"
#include "AnimBoneCompressionCodec_ACLSafe.generated.h"

/** Uses the open source Animation Compression Library with the safest and least destructive settings suitable when animations must be preserved with near raw fidelity. */
UCLASS(MinimalAPI, config = Engine, meta = (DisplayName = "Anim Compress ACL Safe"))
class UAnimBoneCompressionCodec_ACLSafe : public UAnimBoneCompressionCodec_ACLBase
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	// UAnimBoneCompressionCodec implementation
	virtual void PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar) override;

	// UAnimBoneCompressionCodec_ACLBase implementation
	virtual void GetCompressionSettings(const class ITargetPlatform* TargetPlatform, acl::compression_settings& OutSettings) const override;
#endif

	// UAnimBoneCompressionCodec implementation
	virtual void DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs, TArrayView<FTransform>& OutAtoms) const override;
	virtual void DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const override;
};
