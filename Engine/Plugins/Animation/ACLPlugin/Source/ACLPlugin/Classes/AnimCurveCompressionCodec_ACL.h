// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "ACLImpl.h"

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimCurveCompressionCodec.h"
#include "AnimCurveCompressionCodec_ACL.generated.h"

/** Uses the open source Animation Compression Library with default settings suitable for general purpose animation curves. */
UCLASS(MinimalAPI, config = Engine, meta = (DisplayName = "ACL Curves"))
class UAnimCurveCompressionCodec_ACL : public UAnimCurveCompressionCodec
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** The curve precision to target when compressing the animation curves. */
	UPROPERTY(EditAnywhere, Category = "ACL Options", meta = (ClampMin = "0"))
	float CurvePrecision;

	/** The mesh deformation precision to target when compressing morph target animation curves. */
	UPROPERTY(EditAnywhere, Category = "ACL Options", meta = (ClampMin = "0", EditCondition = "MorphTargetSource != nullptr"))
	float MorphTargetPositionPrecision;

	/** The skeletal mesh used to estimate the morph target deformation during compression. */
	UPROPERTY(EditAnywhere, Category = "ACL Options")
	TObjectPtr<class USkeletalMesh> MorphTargetSource;

	//////////////////////////////////////////////////////////////////////////
	// UAnimCurveCompressionCodec implementation
	virtual void PopulateDDCKey(FArchive& Ar) override;
	virtual bool Compress(const FCompressibleAnimData& AnimSeq, FAnimCurveCompressionResult& OutResult) override;
	virtual int64 EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const override;
#endif

	// UAnimCurveCompressionCodec implementation
	virtual bool ValidateCompressedData(UObject* DataOwner, const FCompressedAnimSequence& AnimSeq) const override;
	virtual void DecompressCurves(const FCompressedAnimSequence& AnimSeq, FBlendedCurve& Curves, float CurrentTime) const override;
	virtual float DecompressCurve(const FCompressedAnimSequence& AnimSeq, FName CurveName, float CurrentTime) const override;
};
