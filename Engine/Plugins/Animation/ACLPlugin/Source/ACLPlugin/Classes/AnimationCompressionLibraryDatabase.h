// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "ACLImpl.h"

THIRD_PARTY_INCLUDES_START
#include <acl/decompression/database/database.h>
#include <acl/decompression/database/database_streamer.h>
THIRD_PARTY_INCLUDES_END

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "UObject/PerPlatformProperties.h"
#include "Engine/LatentActionManager.h"
#include "Serialization/BulkData.h"
#include "UObject/ObjectMacros.h"
#include "AnimationCompressionLibraryDatabase.generated.h"

/** An enum to represent the ACL visual fidelity level. */
UENUM()
enum class ACLVisualFidelity : uint8
{
	Highest UMETA(DisplayName = "Highest"),
	Medium UMETA(DisplayName = "Medium"),
	Lowest UMETA(DisplayName = "Lowest"),
};

/** An enum to represent the result of latent visual fidelity change requests. */
UENUM(BlueprintType)
enum class ACLVisualFidelityChangeResult : uint8
{
	Dispatched,
	Completed,
	Failed,
};

/** Represents a pending visual fidelity change request */
struct FFidelityChangeRequest
{
	ACLVisualFidelityChangeResult* Result;	// Optional, present when triggered from a blueprint

	uint32 RequestID;
	ACLVisualFidelity Fidelity;
	bool bIsInProgress;
};

/** An ACL database object references several UAnimSequence instances that it contains. */
UCLASS(MinimalAPI, config = Engine, meta = (DisplayName = "ACL Database"))
class UAnimationCompressionLibraryDatabase : public UObject
{
	GENERATED_UCLASS_BODY()

private:
	/** The raw binary data for our compressed database and anim sequences. Present only in cooked builds. */
	UPROPERTY()
	TArray<uint8> CookedCompressedBytes;

	/** Stores a mapping for each anim sequence, where its compressed data lives in our compressed buffer. Each 64 bit value is split into 32 bits: (Hash << 32) | Offset. Present only in cooked builds. */
	UPROPERTY()
	TArray<uint64> CookedAnimSequenceMappings;

	/** Bulk data that we'll stream. Present only in cooked builds. */
	FByteBulkData CookedBulkData;

	/** The database decompression context object. Bound to the compressed database instance. */
	acl::database_context<UEDefaultDatabaseSettings> DatabaseContext;

	/** The streamer instance used by the database context. Only used in cooked builds. */
	TUniquePtr<acl::database_streamer> DatabaseStreamer;

	/** The current visual fidelity level. */
	ACLVisualFidelity CurrentVisualFidelity;

	/** The next fidelity change request ID. Always increments. */
	uint32 NextFidelityChangeRequestID;

	/** A queue of visual fidelity change requests. */
	TArray<FFidelityChangeRequest> FidelityChangeRequests;

	/** The handle to the fidelity update ticket. */
	FTSTicker::FDelegateHandle FidelityUpdateTickerHandle;

#if WITH_EDITORONLY_DATA
	/** What percentage of the key frames should remain in the anim sequences. */
	UPROPERTY(VisibleAnywhere, Category = "Database")
	float HighestImportanceProportion;

	/** What percentage of the key frames should be moved to the database. Medium importance key frames are moved second. */
	UPROPERTY(EditAnywhere, Category = "Database", meta = (ClampMin = "0", ClampMax = "1"))
	float MediumImportanceProportion;

	/** What percentage of the key frames should be moved to the database. Least important key frames are moved first. */
	UPROPERTY(EditAnywhere, Category = "Database", meta = (ClampMin = "0", ClampMax = "1"))
	float LowestImportanceProportion;

	/** Whether or not to strip the lowest importance tier entirely from disk. Stripping the lowest tier means that the visual fidelity of Highest and Medium are equivalent. */
	UPROPERTY(EditAnywhere, Category = "Database")
	FPerPlatformBool StripLowestImportanceTier;
#endif

	/** The maximum size in KiloBytes of streaming requests. Setting this to 0 will force tiers to load in a single request regardless of their size. */
	UPROPERTY(EditAnywhere, Category = "Database", meta = (ClampMin = "4", ClampMax = "1048576"))
	uint32 MaxStreamRequestSizeKB;

	/** The default level of quality to set when the database loads in-game. By default, nothing is streamed in. */
	UPROPERTY(EditAnywhere, Category = "Database")
	ACLVisualFidelity DefaultVisualFidelity;

#if WITH_EDITORONLY_DATA
	/** The level of quality to preview with the database when decompressing in the editor. */
	UPROPERTY(EditAnywhere, Transient, Category = "Debug")
	ACLVisualFidelity PreviewVisualFidelity;

	/** Editor only, transient, preview version of 'CookedCompressedBytes'. */
	TArray<uint8> PreviewCompressedBytes;

	/** Editor only, transient, preview version of 'CookedAnimSequenceMappings'. */
	TArray<uint64> PreviewAnimSequenceMappings;

	/** Editor only, transient, preview version of 'CookedBulkData'. */
	TArray<uint8> PreviewBulkData;

	/** Editor only, transient, preview version of 'DatabaseStreamer'. */
	TUniquePtr<acl::database_streamer> PreviewDatabaseStreamer;

	/** The anim sequences contained within the database. Built manually from the asset UI, content browser, or with a commandlet. */
	UPROPERTY(VisibleAnywhere, Category = "Metadata")
	TArray<TObjectPtr<class UAnimSequence>> AnimSequences;

	/** The total num of Animation Sequences in this database. */
	UPROPERTY(VisibleAnywhere, Category = "Stats")
	int32 NumAnimSequences;

	/** The total size of all Animation Sequences if the database were not used. */
	UPROPERTY(VisibleAnywhere, Category = "Stats")
	int32 AnimSequencesOldSizeKB;

	/** The total size of all Animation Sequences with the database in use. */
	UPROPERTY(VisibleAnywhere, Category = "Stats")
	int32 AnimSequencesNewSizeKB;

	/** The total size of the database. */
	UPROPERTY(VisibleAnywhere, Category = "Stats")
	int32 DatabaseSizeKB;

	/** The size of the database metadata. */
	UPROPERTY(VisibleAnywhere, Category = "Stats")
	int32 DatabaseMetadataSizeKB;

	/** The size of the database medium importance streaming tier. */
	UPROPERTY(VisibleAnywhere, Category = "Stats")
	int32 MediumImportanceSizeKB;

	/** The size of the database low importance streaming tier before any stripping. */
	UPROPERTY(VisibleAnywhere, Category = "Stats")
	int32 LowImportanceSizeSizeKB;

public:
	//////////////////////////////////////////////////////////////////////////
	// UObject implementation
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;

	/** Updates the internal list of anim sequences that reference this database. Returns whether or not anything changed. */
	ACLPLUGIN_API bool UpdateReferencingAnimSequenceList();
#endif

public:
	// UObject implementation
	virtual void BeginDestroy() override;
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;

	/** Initiate a latent database change in visual fidelity by streaming in/out as necessary. */
	void SetVisualFidelity(ACLVisualFidelity VisualFidelity);

	/** Retrieves the current visual fidelity level. */
	ACLVisualFidelity GetVisualFidelity() const { return CurrentVisualFidelity; }

	/** Initiate a latent database change in quality by streaming in/out as necessary. */
	UFUNCTION(BlueprintCallable, Category = "Animation|ACL", meta = (DisplayName = "Set Database Visual Fidelity", WorldContext="WorldContextObject", Latent, LatentInfo = "LatentInfo", ExpandEnumAsExecs="Result"))
	static void SetVisualFidelity(UObject* WorldContextObject, FLatentActionInfo LatentInfo, UAnimationCompressionLibraryDatabase* DatabaseAsset, ACLVisualFidelityChangeResult& Result, ACLVisualFidelity VisualFidelity = ACLVisualFidelity::Highest);

	UFUNCTION(BlueprintCallable, Category = "Animation|ACL", meta = (DisplayName = "Get Database Visual Fidelity"))
	static ACLVisualFidelity GetVisualFidelity(UAnimationCompressionLibraryDatabase* DatabaseAsset);

private:
#if WITH_EDITORONLY_DATA
	/** Builds our database and its related mappings as well as the new anim sequence data. */
	void BuildDatabase(TArray<uint8>& OutCompressedBytes, TArray<uint64>& OutAnimSequenceMappings, TArray<uint8>& OutBulkData, bool bStripLowestTier = false);

	/** Updates the internal preview state and optionally builds the database when requested. */
	void UpdatePreviewState(bool bBuildDatabase);
#endif

	/** Shared implementation between C++ and blueprint interfaces. Returns the request ID necessary to abort or ~0 if no change is required. */
	uint32 SetVisualFidelityImpl(ACLVisualFidelity VisualFidelity, ACLVisualFidelityChangeResult* OutResult);

	/** Cancels an ongoing visual fidelity request. Only supported if the request hasn't completed. Will not update the ACLVisualFidelityChangeResult of the request. */
	void CancelVisualFidelityRequestImpl(uint32 RequestID);

	/** Core ticker update function to update our visual fidelity state. */
	bool UpdateVisualFidelityTicker(float DeltaTime);

	friend class FACLPlugin;
	friend class FSetDatabaseVisualFidelityAction;
	friend class UAnimBoneCompressionCodec_ACLDatabase;
	friend struct FACLDatabaseCompressedAnimData;
};
