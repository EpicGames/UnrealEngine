// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "Animation/AnimBoneDecompressionData.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "Animation/AnimTypes.h"
#include "AnimEncoding.h"
#include "CoreMinimal.h"

#include "ACLImpl.h"

THIRD_PARTY_INCLUDES_START
#include <acl/decompression/decompress.h>
#include <acl/decompression/database/database.h>
THIRD_PARTY_INCLUDES_END

constexpr acl::sample_rounding_policy get_rounding_policy(EAnimInterpolationType InterpType) { return InterpType == EAnimInterpolationType::Step ? acl::sample_rounding_policy::floor : acl::sample_rounding_policy::none; }

/*
 * The FTransform type does not support setting the members directly from vector types
 * so we derive from it and expose that functionality.
 */
struct FACLTransform final : public FTransform
{
	// Under UE5, these convert from float32 to float64

	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL SetRotationRaw(rtm::quatf_arg0 Rotation_)
	{
#if ENABLE_VECTORIZED_TRANSFORM
		Rotation = Rotation_;
#else
		FQuat4f T;
		rtm::quat_store(Rotation_, &T.X);
		Rotation = (FQuat)T;
#endif
	}

	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL SetTranslationRaw(rtm::vector4f_arg0 Translation_)
	{
#if ENABLE_VECTORIZED_TRANSFORM
		Translation = VectorSet_W0(Translation_);
#else
		FVector3f T;		// LWC_TODO: Perf pessimization.
		rtm::vector_store3(Translation_, &T.X);
		Translation = (FVector)T;
#endif
	}

	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL SetScale3DRaw(rtm::vector4f_arg0 Scale_)
	{
#if ENABLE_VECTORIZED_TRANSFORM
		Scale3D = VectorSet_W0(Scale_);
#else
		FVector3f S;		// LWC_TODO: Perf pessimization.
		rtm::vector_store3(Scale_, &S.X);
		Scale3D = (FVector)S;
#endif
	}
};

/** These 3 indices map into the output Atom array. */
struct FAtomIndices
{
	uint16 Rotation;
	uint16 Translation;
	uint16 Scale;
};

/*
 * Output pose writer that can selectively skip certain tracks.
 */
template<bool bUseBindPose>
struct FUEOutputWriter final : public acl::track_writer
{
	// Raw pointer for performance reasons, caller is responsible for ensuring data is valid

	// The bind pose
	const FTransform* RefPoses;

	// Maps track indices to bone indices
	const FTrackToSkeletonMap* TrackToBoneMapping;

	// The track to output transform index map
	const FAtomIndices* TrackToAtomsMap;

	// The output transforms we write
	FACLTransform* Atoms;

	FUEOutputWriter(const FAtomIndices* TrackToAtomsMap_, TArrayView<FTransform>& Atoms_)
		: RefPoses(nullptr)
		, TrackToBoneMapping(nullptr)
		, TrackToAtomsMap(TrackToAtomsMap_)
		, Atoms(static_cast<FACLTransform*>(Atoms_.GetData()))
	{}

	FUEOutputWriter(const TArrayView<const FTransform>& RefPoses_, const TArrayView<const FTrackToSkeletonMap>& TrackToSkeletonMap, const FAtomIndices* TrackToAtomsMap_, TArrayView<FTransform>& Atoms_)
		: RefPoses(RefPoses_.GetData())
		, TrackToBoneMapping(TrackToSkeletonMap.GetData())
		, TrackToAtomsMap(TrackToAtomsMap_)
		, Atoms(static_cast<FACLTransform*>(Atoms_.GetData()))
	{}

	//////////////////////////////////////////////////////////////////////////
	// Override the OutputWriter behavior
	// If we use the bind pose, each default sub-track will grab the bind pose value
	// Otherwise we output the constant default values
	FORCEINLINE_DEBUGGABLE bool skip_track_rotation(uint32_t BoneIndex) const { return TrackToAtomsMap[BoneIndex].Rotation == 0xFFFF; }
	FORCEINLINE_DEBUGGABLE bool skip_track_translation(uint32_t BoneIndex) const { return TrackToAtomsMap[BoneIndex].Translation == 0xFFFF; }
	FORCEINLINE_DEBUGGABLE bool skip_track_scale(uint32_t BoneIndex) const { return TrackToAtomsMap[BoneIndex].Scale == 0xFFFF; }

	static constexpr acl::default_sub_track_mode get_default_rotation_mode() { return bUseBindPose ? acl::default_sub_track_mode::variable : acl::default_sub_track_mode::constant; }
	static constexpr acl::default_sub_track_mode get_default_translation_mode() { return bUseBindPose ? acl::default_sub_track_mode::variable : acl::default_sub_track_mode::constant; }
	static constexpr acl::default_sub_track_mode get_default_scale_mode() { return bUseBindPose ? acl::default_sub_track_mode::variable : acl::default_sub_track_mode::legacy; }

	// TODO: There is a performance impact here because the ref pose might use doubles on PC
	FORCEINLINE_DEBUGGABLE rtm::quatf RTM_SIMD_CALL get_variable_default_rotation(uint32_t TrackIndex) const
	{
		return UEQuatToACL(RefPoses[TrackToBoneMapping[TrackIndex].BoneTreeIndex].GetRotation());
	}

	FORCEINLINE_DEBUGGABLE rtm::vector4f RTM_SIMD_CALL get_variable_default_translation(uint32_t TrackIndex) const
	{
		return UEVector3ToACL(RefPoses[TrackToBoneMapping[TrackIndex].BoneTreeIndex].GetTranslation());
	}

	FORCEINLINE_DEBUGGABLE rtm::vector4f RTM_SIMD_CALL get_variable_default_scale(uint32_t TrackIndex) const
	{
		return UEVector3ToACL(RefPoses[TrackToBoneMapping[TrackIndex].BoneTreeIndex].GetScale3D());
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a quaternion rotation value for a specified bone index
	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_rotation(uint32_t BoneIndex, rtm::quatf_arg0 Rotation)
	{
		const uint32 AtomIndex = TrackToAtomsMap[BoneIndex].Rotation;

		FACLTransform& BoneAtom = Atoms[AtomIndex];
		BoneAtom.SetRotationRaw(Rotation);
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a translation value for a specified bone index
	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_translation(uint32_t BoneIndex, rtm::vector4f_arg0 Translation)
	{
		const uint32 AtomIndex = TrackToAtomsMap[BoneIndex].Translation;

		FACLTransform& BoneAtom = Atoms[AtomIndex];
		BoneAtom.SetTranslationRaw(Translation);
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a scale value for a specified bone index
	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_scale(uint32_t BoneIndex, rtm::vector4f_arg0 Scale)
	{
		const uint32 AtomIndex = TrackToAtomsMap[BoneIndex].Scale;

		FACLTransform& BoneAtom = Atoms[AtomIndex];
		BoneAtom.SetScale3DRaw(Scale);
	}
};

/*
 * Output pose writer that can selectively skip certain tracks.
 */
template<bool bUseBindPose>
struct FUEOutputWriterSoA final : public acl::track_writer
{
	// Raw pointer for performance reasons, caller is responsible for ensuring data is valid

	// The bind pose
	const FTransform* RefPoses;

	// Maps track indices to bone indices
	const FTrackToSkeletonMap* TrackToBoneMapping;

	// The track to output transform index map
	const FAtomIndices* TrackToAtomsMap;

	// The output transforms we write
	FQuat* Rotations;
	FVector* Translations;
	FVector* Scales3D;

	FUEOutputWriterSoA(
		const FAtomIndices* InTrackToAtomsMap,
		TArrayView<FQuat>& InRotations,
		TArrayView<FVector>& InTranslations,
		TArrayView<FVector>& InScales3D)
		: RefPoses(nullptr)
		, TrackToBoneMapping(nullptr)
		, TrackToAtomsMap(InTrackToAtomsMap)
		, Rotations(InRotations.GetData())
		, Translations(InTranslations.GetData())
		, Scales3D(InScales3D.GetData())
	{}

	FUEOutputWriterSoA(
		const TArrayView<const FTransform>& InRefPoses,
		const TArrayView<const FTrackToSkeletonMap>& TrackToSkeletonMap,
		const FAtomIndices* InTrackToAtomsMap,
		TArrayView<FQuat>& InRotations,
		TArrayView<FVector>& InTranslations,
		TArrayView<FVector>& InScales3D)
		: RefPoses(InRefPoses.GetData())
		, TrackToBoneMapping(TrackToSkeletonMap.GetData())
		, TrackToAtomsMap(InTrackToAtomsMap)
		, Rotations(InRotations.GetData())
		, Translations(InTranslations.GetData())
		, Scales3D(InScales3D.GetData())
	{}

	//////////////////////////////////////////////////////////////////////////
	// Override the OutputWriter behavior
	// If we use the bind pose, each default sub-track will grab the bind pose value
	// Otherwise we output the constant default values
	FORCEINLINE_DEBUGGABLE bool skip_track_rotation(uint32_t BoneIndex) const { return TrackToAtomsMap[BoneIndex].Rotation == 0xFFFF; }
	FORCEINLINE_DEBUGGABLE bool skip_track_translation(uint32_t BoneIndex) const { return TrackToAtomsMap[BoneIndex].Translation == 0xFFFF; }
	FORCEINLINE_DEBUGGABLE bool skip_track_scale(uint32_t BoneIndex) const { return TrackToAtomsMap[BoneIndex].Scale == 0xFFFF; }

	static constexpr acl::default_sub_track_mode get_default_rotation_mode() { return bUseBindPose ? acl::default_sub_track_mode::variable : acl::default_sub_track_mode::constant; }
	static constexpr acl::default_sub_track_mode get_default_translation_mode() { return bUseBindPose ? acl::default_sub_track_mode::variable : acl::default_sub_track_mode::constant; }
	static constexpr acl::default_sub_track_mode get_default_scale_mode() { return bUseBindPose ? acl::default_sub_track_mode::variable : acl::default_sub_track_mode::legacy; }

	// TODO: There is a performance impact here because the ref pose might use doubles on PC
	FORCEINLINE_DEBUGGABLE rtm::quatf RTM_SIMD_CALL get_variable_default_rotation(uint32_t TrackIndex) const
	{
		return UEQuatToACL(RefPoses[TrackToBoneMapping[TrackIndex].BoneTreeIndex].GetRotation());
	}

	FORCEINLINE_DEBUGGABLE rtm::vector4f RTM_SIMD_CALL get_variable_default_translation(uint32_t TrackIndex) const
	{
		return UEVector3ToACL(RefPoses[TrackToBoneMapping[TrackIndex].BoneTreeIndex].GetTranslation());
	}

	FORCEINLINE_DEBUGGABLE rtm::vector4f RTM_SIMD_CALL get_variable_default_scale(uint32_t TrackIndex) const
	{
		return UEVector3ToACL(RefPoses[TrackToBoneMapping[TrackIndex].BoneTreeIndex].GetScale3D());
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a quaternion rotation value for a specified bone index
	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_rotation(uint32_t BoneIndex, rtm::quatf_arg0 Rotation)
	{
		const uint32 AtomIndex = TrackToAtomsMap[BoneIndex].Rotation;

		FQuat& OutRotation = Rotations[AtomIndex];
		OutRotation = (FQuat)ACLQuatToUE(Rotation);
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a translation value for a specified bone index
	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_translation(uint32_t BoneIndex, rtm::vector4f_arg0 Translation)
	{
		const uint32 AtomIndex = TrackToAtomsMap[BoneIndex].Translation;

		FVector& OutTranslation = Translations[AtomIndex];
		OutTranslation = (FVector)ACLVector3ToUE(Translation);
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a scale value for a specified bone index
	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_scale(uint32_t BoneIndex, rtm::vector4f_arg0 Scale)
	{
		const uint32 AtomIndex = TrackToAtomsMap[BoneIndex].Scale;

		FVector& OutScale = Scales3D[AtomIndex];
		OutScale = (FVector)ACLVector3ToUE(Scale);
	}
};

/*
* Output track writer for a single track.
*/
template<bool bUseBindPose>
struct UEOutputTrackWriter final : public acl::track_writer
{
	// The bind pose
	const FTransform* RefPoses;

	// Maps track indices to bone indices
	const FTrackToSkeletonMap* TrackToBoneMapping;

	// Raw reference for performance reasons, caller is responsible for ensuring data is valid
	FACLTransform& Atom;

	explicit UEOutputTrackWriter(FTransform& Atom_)
		: RefPoses(nullptr)
		, TrackToBoneMapping(nullptr)
		, Atom(static_cast<FACLTransform&>(Atom_))
	{}

	UEOutputTrackWriter(const TArrayView<const FTransform>& RefPoses_, const TArrayView<const FTrackToSkeletonMap>& TrackToSkeletonMap, FTransform& Atom_)
		: RefPoses(RefPoses_.GetData())
		, TrackToBoneMapping(TrackToSkeletonMap.GetData())
		, Atom(static_cast<FACLTransform&>(Atom_))
	{}

	// TODO: UE5 FTransform uses doubles, we should convert the default values to use doubles to speed it up
	// But to do this, we have to add support for it in ACL:
	//    - We have to use 'auto' when we cache the value we query from the writer
	//    - We have to use a new compile time parameter in the writer, if float64 output is enabled, etc
	//      ACL would benefit from knowing this as we could convert floats in SOA form before caching but maybe not worth it

	//////////////////////////////////////////////////////////////////////////
	// Override the OutputWriter behavior
	// If we use the bind pose, each default sub-track will grab the bind pose value
	// Otherwise we output the constant default values
	static constexpr acl::default_sub_track_mode get_default_rotation_mode() { return bUseBindPose ? acl::default_sub_track_mode::variable : acl::default_sub_track_mode::constant; }
	static constexpr acl::default_sub_track_mode get_default_translation_mode() { return bUseBindPose ? acl::default_sub_track_mode::variable : acl::default_sub_track_mode::constant; }
	static constexpr acl::default_sub_track_mode get_default_scale_mode() { return bUseBindPose ? acl::default_sub_track_mode::variable : acl::default_sub_track_mode::legacy; }

	// TODO: There is a performance impact here because the ref pose might use doubles on PC
	FORCEINLINE_DEBUGGABLE rtm::quatf RTM_SIMD_CALL get_variable_default_rotation(uint32_t TrackIndex) const
	{
		return UEQuatToACL(RefPoses[TrackToBoneMapping[TrackIndex].BoneTreeIndex].GetRotation());
	}

	FORCEINLINE_DEBUGGABLE rtm::vector4f RTM_SIMD_CALL get_variable_default_translation(uint32_t TrackIndex) const
	{
		return UEVector3ToACL(RefPoses[TrackToBoneMapping[TrackIndex].BoneTreeIndex].GetTranslation());
	}

	FORCEINLINE_DEBUGGABLE rtm::vector4f RTM_SIMD_CALL get_variable_default_scale(uint32_t TrackIndex) const
	{
		return UEVector3ToACL(RefPoses[TrackToBoneMapping[TrackIndex].BoneTreeIndex].GetScale3D());
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a quaternion rotation value for a specified bone index
	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_rotation(uint32_t BoneIndex, rtm::quatf_arg0 Rotation)
	{
		Atom.SetRotationRaw(Rotation);
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a translation value for a specified bone index
	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_translation(uint32_t BoneIndex, rtm::vector4f_arg0 Translation)
	{
		Atom.SetTranslationRaw(Translation);
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a scale value for a specified bone index
	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_scale(uint32_t BoneIndex, rtm::vector4f_arg0 Scale)
	{
		Atom.SetScale3DRaw(Scale);
	}
};

template<class ACLContextType>
FORCEINLINE_DEBUGGABLE void DecompressBone(FAnimSequenceDecompressionContext& DecompContext, ACLContextType& ACLContext, int32 TrackIndex, FTransform& OutAtom)
{
	const float Time = DecompContext.GetEvaluationTime();

	ACLContext.seek(Time, get_rounding_policy(DecompContext.Interpolation));

	const acl::compressed_tracks* CompressedClipData = ACLContext.get_compressed_tracks();

	// See [Bind pose stripping] for details
	// Are we non-additive?
	if (CompressedClipData->get_default_scale() != 0)
	{
		// Non-additive anim sequences must write out the bind pose for default sub-tracks since they
		// have been stripped from the data. Additive anim sequences always have the additive identity
		// stripped, no need for the bind pose.
		constexpr bool bUseBindPose = true;

		checkf(DecompContext.GetRefLocalPoses().Num() > 0, TEXT("Reference pose must be provided in the FAnimSequenceDecompressionContext constructor to use bind pose stripping"));
		checkf(DecompContext.GetTrackToSkeletonMap().Num() > 0, TEXT("TrackToSkeletonMap must be provided in the FAnimSequenceDecompressionContext constructor to use bind pose stripping"));

		UEOutputTrackWriter<bUseBindPose> Writer(DecompContext.GetRefLocalPoses(), DecompContext.GetTrackToSkeletonMap(), OutAtom);
		ACLContext.decompress_track(TrackIndex, Writer);
	}
	else
	{
		// Additive anim sequences have the identity stripped out, we'll output it.
		// We also output the regular identity if bind pose stripping isn't enabled.
		constexpr bool bUseBindPose = false;

		UEOutputTrackWriter<bUseBindPose> Writer(OutAtom);
		ACLContext.decompress_track(TrackIndex, Writer);
	}
}

template<class ACLContextType>
FORCEINLINE_DEBUGGABLE void DecompressPose(FAnimSequenceDecompressionContext& DecompContext, ACLContextType& ACLContext,
	const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs,
	TArrayView<FTransform>& OutAtoms)
{
	const float Time = DecompContext.GetEvaluationTime();

	// Seek first, we'll start prefetching ahead right away
	ACLContext.seek(Time, get_rounding_policy(DecompContext.Interpolation));

	const acl::compressed_tracks* CompressedClipData = ACLContext.get_compressed_tracks();
	const int32 ACLBoneCount = CompressedClipData->get_num_tracks();

	// TODO: Allocate this with padding and use SIMD to set everything to 0xFF
	FAtomIndices* TrackToAtomsMap = new(FMemStack::Get()) FAtomIndices[ACLBoneCount];
	FMemory::Memset(TrackToAtomsMap, 0xFF, sizeof(FAtomIndices) * ACLBoneCount);

	// TODO: We should only need 1x uint16 atom index for each track/bone index
	// and we need 3 bits to tell whether we care about the rot/trans/scale
	// The rot and scale pairs are often the same array, skip the scale iteration and set both flags while
	// iterating on the rotation pairs. This will reduce the mapping size by 2 bytes if we use 4 bytes.
	// All reads will be aligned, can we pack further with 1x uint16 and 1x uint8 and do unaligned loads?
	// Need to double check what the ASM looks like on x64 and ARM first to make sure it's good
	// Maybe having two arrays side by side is better with uint16/uint8? Or having 1 bitset array?
	// Ultimately, when we load these indices, they will be in the L1 since we write them here just before
	// we use them during decompression. Optimizing for quick loading/unpacking it best.

#if DO_CHECK
	int32 MinAtomIndex = OutAtoms.Num();
	int32 MaxAtomIndex = -1;
	int32 MinTrackIndex = INT_MAX;
	int32 MaxTrackIndex = -1;
#endif

	for (const BoneTrackPair& Pair : RotationPairs)
	{
		TrackToAtomsMap[Pair.TrackIndex].Rotation = (uint16)Pair.AtomIndex;

#if DO_CHECK
		MinAtomIndex = FMath::Min(MinAtomIndex, Pair.AtomIndex);
		MaxAtomIndex = FMath::Max(MaxAtomIndex, Pair.AtomIndex);
		MinTrackIndex = FMath::Min(MinTrackIndex, Pair.TrackIndex);
		MaxTrackIndex = FMath::Max(MaxTrackIndex, Pair.TrackIndex);
#endif
	}

	for (const BoneTrackPair& Pair : TranslationPairs)
	{
		TrackToAtomsMap[Pair.TrackIndex].Translation = (uint16)Pair.AtomIndex;

#if DO_CHECK
		MinAtomIndex = FMath::Min(MinAtomIndex, Pair.AtomIndex);
		MaxAtomIndex = FMath::Max(MaxAtomIndex, Pair.AtomIndex);
		MinTrackIndex = FMath::Min(MinTrackIndex, Pair.TrackIndex);
		MaxTrackIndex = FMath::Max(MaxTrackIndex, Pair.TrackIndex);
#endif
	}

	const acl::acl_impl::tracks_header& TracksHeader = acl::acl_impl::get_tracks_header(*CompressedClipData);
	if (TracksHeader.get_has_scale())
	{
		for (const BoneTrackPair& Pair : ScalePairs)
		{
			TrackToAtomsMap[Pair.TrackIndex].Scale = (uint16)Pair.AtomIndex;

#if DO_CHECK
			MinAtomIndex = FMath::Min(MinAtomIndex, Pair.AtomIndex);
			MaxAtomIndex = FMath::Max(MaxAtomIndex, Pair.AtomIndex);
			MinTrackIndex = FMath::Min(MinTrackIndex, Pair.TrackIndex);
			MaxTrackIndex = FMath::Max(MaxTrackIndex, Pair.TrackIndex);
#endif
		}
	}

#if DO_CHECK
	// Only assert once for performance reasons, when we write the pose, we won't perform the checks
	checkf(OutAtoms.IsValidIndex(MinAtomIndex), TEXT("Invalid atom index: %d"), MinAtomIndex);
	checkf(OutAtoms.IsValidIndex(MaxAtomIndex), TEXT("Invalid atom index: %d"), MaxAtomIndex);
	checkf(MinTrackIndex >= 0, TEXT("Invalid track index: %d"), MinTrackIndex);
	checkf(MaxTrackIndex < ACLBoneCount, TEXT("Invalid track index: %d"), MaxTrackIndex);
#endif

	// We will decompress the whole pose even if we only care about a smaller subset of bone tracks.
	// This ensures we read the compressed pose data once, linearly.

	// See [Bind pose stripping] for details
	// Are we non-additive?
	if (CompressedClipData->get_default_scale() != 0)
	{
		// Non-additive anim sequences must write out the bind pose for default sub-tracks since they
		// have been stripped from the data. Additive anim sequences always have the additive identity
		// stripped, no need for the bind pose.
		constexpr bool bUseBindPose = true;

		FUEOutputWriter<bUseBindPose> PoseWriter(DecompContext.GetRefLocalPoses(), DecompContext.GetTrackToSkeletonMap(), TrackToAtomsMap, OutAtoms);
		ACLContext.decompress_tracks(PoseWriter);
	}
	else
	{
		// Additive anim sequences have the identity stripped out, we'll output it.
		// We also output the regular identity if bind pose stripping isn't enabled.
		constexpr bool bUseBindPose = false;

		FUEOutputWriter<bUseBindPose> PoseWriter(TrackToAtomsMap, OutAtoms);
		ACLContext.decompress_tracks(PoseWriter);
	}
}

template<class ACLContextType>
FORCEINLINE_DEBUGGABLE void DecompressPose(
	const FAnimSequenceDecompressionContext& DecompContext,
	ACLContextType& ACLContext,
	const UE::Anim::FAnimPoseDecompressionData& DecompressionData)
{
	const float Time = DecompContext.GetEvaluationTime();

	// Seek first, we'll start prefetching ahead right away
	ACLContext.seek(Time, get_rounding_policy(DecompContext.Interpolation));

	const acl::compressed_tracks* CompressedClipData = ACLContext.get_compressed_tracks();
	const int32 TrackCount = CompressedClipData->get_num_tracks();

	// TODO: Allocate this with padding and use SIMD to set everything to 0xFF
	FAtomIndices* TrackToAtomsMap = new(FMemStack::Get()) FAtomIndices[TrackCount];
	FMemory::Memset(TrackToAtomsMap, 0xFF, sizeof(FAtomIndices) * TrackCount);

	// TODO: We should only need 1x uint16 atom index for each track/bone index
	// and we need 3 bits to tell whether we care about the rot/trans/scale
	// The rot and scale pairs are often the same array, skip the scale iteration and set both flags while
	// iterating on the rotation pairs. This will reduce the mapping size by 2 bytes if we use 4 bytes.
	// All reads will be aligned, can we pack further with 1x uint16 and 1x uint8 and do unaligned loads?
	// Need to double check what the ASM looks like on x64 and ARM first to make sure it's good
	// Maybe having two arrays side by side is better with uint16/uint8? Or having 1 bitset array?
	// Ultimately, when we load these indices, they will be in the L1 since we write them here just before
	// we use them during decompression. Optimizing for quick loading/unpacking it best.

	for (const BoneTrackPair& Pair : DecompressionData.GetRotationPairs())
	{
		TrackToAtomsMap[Pair.TrackIndex].Rotation = (uint16)Pair.AtomIndex;
	}

	for (const BoneTrackPair& Pair : DecompressionData.GetTranslationPairs())
	{
		TrackToAtomsMap[Pair.TrackIndex].Translation = (uint16)Pair.AtomIndex;
	}

	const acl::acl_impl::tracks_header& TracksHeader = acl::acl_impl::get_tracks_header(*CompressedClipData);
	if (TracksHeader.get_has_scale())
	{
		for (const BoneTrackPair& Pair : DecompressionData.GetScalePairs())
		{
			TrackToAtomsMap[Pair.TrackIndex].Scale = (uint16)Pair.AtomIndex;
		}
	}

	// We will decompress the whole pose even if we only care about a smaller subset of bone tracks.
	// This ensures we read the compressed pose data once, linearly.

	// See [Bind pose stripping] for details
	// Are we non-additive?
	if (CompressedClipData->get_default_scale() != 0)
	{
		// Non-additive anim sequences must write out the bind pose for default sub-tracks since they
		// have been stripped from the data. Additive anim sequences always have the additive identity
		// stripped, no need for the bind pose.
		constexpr bool bUseBindPose = true;

		FUEOutputWriterSoA<bUseBindPose> PoseWriter(DecompContext.GetRefLocalPoses(), DecompContext.GetTrackToSkeletonMap(), TrackToAtomsMap, DecompressionData.GetOutAtomRotations(), DecompressionData.GetOutAtomTranslations(), DecompressionData.GetOutAtomScales3D());
		ACLContext.decompress_tracks(PoseWriter);
	}
	else
	{
		// Additive anim sequences have the identity stripped out, we'll output it.
		// We also output the regular identity if bind pose stripping isn't enabled.
		constexpr bool bUseBindPose = false;

		FUEOutputWriterSoA<bUseBindPose> PoseWriter(TrackToAtomsMap, DecompressionData.GetOutAtomRotations(), DecompressionData.GetOutAtomTranslations(), DecompressionData.GetOutAtomScales3D());
		ACLContext.decompress_tracks(PoseWriter);
	}
}
