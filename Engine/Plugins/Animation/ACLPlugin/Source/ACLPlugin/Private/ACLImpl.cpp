// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "ACLImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ACLImpl)

#if WITH_EDITOR
#include "AnimationCompression.h"
#include "AnimationUtils.h"
#include "UObject/PerPlatformProperties.h"
#include "Animation/AnimCompressionTypes.h"
#include "Interfaces/ITargetPlatform.h"
#include "PlatformInfo.h"

acl::rotation_format8 GetRotationFormat(ACLRotationFormat Format)
{
	switch (Format)
	{
	default:
	case ACLRF_Quat_128:			return acl::rotation_format8::quatf_full;
	case ACLRF_QuatDropW_96:		return acl::rotation_format8::quatf_drop_w_full;
	case ACLRF_QuatDropW_Variable:	return acl::rotation_format8::quatf_drop_w_variable;
	}
}

acl::vector_format8 GetVectorFormat(ACLVectorFormat Format)
{
	switch (Format)
	{
	default:
	case ACLVF_Vector3_96:			return acl::vector_format8::vector3f_full;
	case ACLVF_Vector3_Variable:	return acl::vector_format8::vector3f_variable;
	}
}

acl::compression_level8 GetCompressionLevel(ACLCompressionLevel Level)
{
	switch (Level)
	{
	default:
	case ACLCL_Lowest:		return acl::compression_level8::lowest;
	case ACLCL_Low:			return acl::compression_level8::low;
	case ACLCL_Medium:		return acl::compression_level8::medium;
	case ACLCL_High:		return acl::compression_level8::high;
	case ACLCL_Highest:		return acl::compression_level8::highest;
	case ACLCL_Automatic:	return acl::compression_level8::automatic;
	}
}

static int32 FindAnimationTrackIndex(const FCompressibleAnimData& CompressibleAnimData, int32 BoneIndex)
{
	const TArray<FTrackToSkeletonMap>& TrackToSkelMap = CompressibleAnimData.TrackToSkeletonMapTable;
	if (BoneIndex != INDEX_NONE)
	{
		for (int32 TrackIndex = 0; TrackIndex < TrackToSkelMap.Num(); ++TrackIndex)
		{
			const FTrackToSkeletonMap& TrackToSkeleton = TrackToSkelMap[TrackIndex];
			if (TrackToSkeleton.BoneTreeIndex == BoneIndex)
				return TrackIndex;
		}
	}

	return INDEX_NONE;
}

static bool IsAdditiveBakedIntoRaw(const FCompressibleAnimData& CompressibleAnimData)
{
	if (!CompressibleAnimData.bIsValidAdditive)
	{
		return true;	// Sequences that aren't additive don't need baking as they are already baked
	}

	if (CompressibleAnimData.RawAnimationData.Num() == 0)
	{
		return true;	// Sequences with no raw data don't need baking and we can't tell if they are baked or not so assume that they are, it doesn't matter
	}

	if (CompressibleAnimData.AdditiveBaseAnimationData.Num() != 0)
	{
		return true;	// Sequence has raw data and additive base data, it is baked
	}

	return false;	// Sequence has raw data but no additive base data, it isn't baked
}

// Returns a bit array, a bit is true if the corresponding UE track has a skeleton bone mapped to it, false otherwise
static TBitArray<> GetMappedUETracks(const FCompressibleAnimData& CompressibleAnimData)
{
	const int32 NumBones = CompressibleAnimData.BoneData.Num();
	const int32 NumUETracks = CompressibleAnimData.TrackToSkeletonMapTable.Num();

	TBitArray<> MappedUETracks(false, NumUETracks);

	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const int32 UETrackIndex = FindAnimationTrackIndex(CompressibleAnimData, BoneIndex);

		if (UETrackIndex >= 0)
		{
			// This track has a bone mapped to it
			MappedUETracks[UETrackIndex] = true;
		}
	}

	return MappedUETracks;
}

acl::track_array_qvvf BuildACLTransformTrackArray(ACLAllocator& AllocatorImpl, const FCompressibleAnimData& CompressibleAnimData,
	float DefaultVirtualVertexDistance, float SafeVirtualVertexDistance,
	bool bBuildAdditiveBase, ACLPhantomTrackMode PhantomTrackMode)
{
	const bool bIsAdditive = bBuildAdditiveBase ? false : CompressibleAnimData.bIsValidAdditive;
	const bool bIsAdditiveBakedIntoRaw = IsAdditiveBakedIntoRaw(CompressibleAnimData);
	check(!bIsAdditive || bIsAdditiveBakedIntoRaw);
	if (bIsAdditive && !bIsAdditiveBakedIntoRaw)
	{
		UE_LOGF(LogAnimationCompression, Fatal, "Animation sequence is additive but it is not baked into the raw data, this is not supported. [%ls]", *CompressibleAnimData.FullName);
		return acl::track_array_qvvf();
	}

	// Ordinary non additive sequences only contain raw data returned by GetRawAnimationData().
	//
	// Additive sequences have their raw data baked with the raw additive base already taken out. At runtime,
	// the additive sequence data returned by GetRawAnimationData() gets applied on top of the
	// additive base sequence whose's data is returned by GetAdditiveBaseAnimationData(). In practice,
	// at runtime the additive base might also be compressed and thus there might be some added noise but
	// it typically isn't visible.
	// The baked additive data should have the same number of frames as the current sequence but if we only care about a specific frame,
	// we use it. When this happens, the additive base will contain that single frame repeated over and over: an animated static pose.
	// To avoid wasting memory, we just grab the first frame.

	const TArray<FRawAnimSequenceTrack>& RawTracks = bBuildAdditiveBase ? CompressibleAnimData.AdditiveBaseAnimationData : CompressibleAnimData.RawAnimationData;
	const uint32 NumSamples = GetNumSamples(CompressibleAnimData);
	const bool bIsStaticPose = NumSamples <= 1 || CompressibleAnimData.SequenceLength < 0.0001f;
	const float SampleRate = bIsStaticPose ? 30.0f : (float(NumSamples - 1) / CompressibleAnimData.SequenceLength);
	const int32 NumBones = CompressibleAnimData.BoneData.Num();
	const int32 NumUETracks = CompressibleAnimData.TrackToSkeletonMapTable.Num();

	// Additive animations have 0,0,0 scale as the default since we add it
	const FVector3f UEDefaultScale(bIsAdditive ? 0.0f : 1.0f);
	const rtm::vector4f ACLDefaultScale = rtm::vector_set(bIsAdditive ? 0.0f : 1.0f);

	rtm::qvvf ACLDefaultAdditiveBindTransform = rtm::qvv_identity();
	ACLDefaultAdditiveBindTransform.scale = ACLDefaultScale;

	// A bit array to tell which UE tracks are mapped to a skeleton bone
	const TBitArray<> MappedUETracks = GetMappedUETracks(CompressibleAnimData);

	// We need to make sure to allocate enough ACL tracks. It is very common to have a skeleton with a number of bones
	// and to have anim sequences that use that skeleton that have fewer tracks. This might happen if bones are added
	// and when this happens, we'll populate the bind pose for that missing track. A less common case can happen where
	// a sequence has compressed tracks for bones that no longer exist. We still need to compress this unused data to
	// ensure the track indices remain in sync with the CompressedTrackToSkeletonMapTable.
	// 
	// We need to allocate one ACL track per UE bone and one for each unmapped UE track
	const int32 NumUnmappedUETracks = NumUETracks - MappedUETracks.CountSetBits();
	const int32 NumInputACLTracks = NumBones + NumUnmappedUETracks;
	int32 NumOutputACLTracks = 0;	// For sanity check

	acl::track_array_qvvf Tracks(AllocatorImpl, NumInputACLTracks);
	Tracks.set_name(acl::string(AllocatorImpl, TCHAR_TO_ANSI(*CompressibleAnimData.FullName)));

	// Populate all our track based on our skeleton, even if some end up stripped
	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const FBoneData& UEBone = CompressibleAnimData.BoneData[BoneIndex];

		acl::track_desc_transformf Desc;

		// We use a higher virtual vertex distance when bones have a socket attached or are keyed end effectors (IK, hand, camera, etc)
		Desc.shell_distance = (UEBone.bHasSocket || UEBone.bKeyEndEffector) ? SafeVirtualVertexDistance : DefaultVirtualVertexDistance;

		const int32 ParentBoneIndex = UEBone.GetParent();
		Desc.parent_index = ParentBoneIndex >= 0 ? ParentBoneIndex : acl::k_invalid_track_index;

		const int32 UETrackIndex = FindAnimationTrackIndex(CompressibleAnimData, BoneIndex);

		// We output bone data in UE track order. If a track isn't present, we will use the bind pose and strip it from the
		// compressed stream.
		if (UETrackIndex >= 0)
		{
			Desc.output_index = UETrackIndex;
			NumOutputACLTracks++;
		}
		else
		{
			Desc.output_index = acl::k_invalid_track_index;
		}

		// Make sure the default scale value is consistent whether we are additive or not
		Desc.default_value.scale = ACLDefaultScale;

		acl::track_qvvf Track = acl::track_qvvf::make_reserve(Desc, AllocatorImpl, NumSamples, SampleRate);
		Track.set_name(acl::string(AllocatorImpl, TCHAR_TO_ANSI(*UEBone.Name.ToString())));

		if (UETrackIndex >= 0)
		{
			// We have a track for this bone, use it
			const FRawAnimSequenceTrack& RawTrack = RawTracks[UETrackIndex];

			for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				const FRawAnimTrackQuat& RotationSample = RawTrack.RotKeys.Num() == 1 ? RawTrack.RotKeys[0] : RawTrack.RotKeys[SampleIndex];
				Track[SampleIndex].rotation = UEQuatToACL(RotationSample);

				const FRawAnimTrackVector3& TranslationSample = RawTrack.PosKeys.Num() == 1 ? RawTrack.PosKeys[0] : RawTrack.PosKeys[SampleIndex];
				Track[SampleIndex].translation = UEVector3ToACL(TranslationSample);

				const FRawAnimTrackVector3& ScaleSample = RawTrack.ScaleKeys.Num() == 0 ? UEDefaultScale : (RawTrack.ScaleKeys.Num() == 1 ? RawTrack.ScaleKeys[0] : RawTrack.ScaleKeys[SampleIndex]);
				Track[SampleIndex].scale = UEVector3ToACL(ScaleSample);
			}
		}
		else
		{
			// No track data for this bone, it must be new. Use the bind pose instead.
			// Additive animations have the identity with 0 scale as their bind pose.
			rtm::qvvf BindTransform;
			if (bIsAdditive)
			{
				BindTransform = ACLDefaultAdditiveBindTransform;
			}
			else
			{
				BindTransform = rtm::qvv_set(UEQuatToACL(UEBone.Orientation), UEVector3ToACL(UEBone.Position), UEVector3ToACL(UEBone.Scale));
			}

			for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				Track[SampleIndex] = BindTransform;
			}
		}

		Tracks[BoneIndex] = MoveTemp(Track);
	}

	// If we have leftover phantom tracks that do not map to any bone in our skeleton, compress them anyway to keep indices consistent
	// Phantom UE tracks have no associated skeleton bone and as such can only be queried at runtime individually with DecompressBone.
	// It should be safe to strip and collapse them to the identity.
	int32 ACLTrackIndex = NumBones;	// Start inserting at the end
	for (int32 UETrackIndex = 0; UETrackIndex < NumUETracks; ++UETrackIndex)
	{
		if (MappedUETracks[UETrackIndex])
		{
			// This track has been populated, skip it
			continue;
		}

		// This track has no corresponding skeleton bone, add it anyway
		acl::track_desc_transformf Desc;

		// Without a bone, we have no way of knowing if we require special treatment
		Desc.shell_distance = DefaultVirtualVertexDistance;

		// Without a bone, no way to know if we have a parent, assume we are a root bone
		Desc.parent_index = acl::k_invalid_track_index;

		// Output index is our UE track index
		Desc.output_index = UETrackIndex;
		NumOutputACLTracks++;

		acl::track_qvvf Track = acl::track_qvvf::make_reserve(Desc, AllocatorImpl, NumSamples, SampleRate);

		if (PhantomTrackMode == ACLPhantomTrackMode::Strip)
		{
			// We'll collapse the track to the identity transform
			// It will be compressed to maintain the track ordering but it will only use 6 bits in total
			const rtm::qvvf IdentityTransform = bIsAdditive ? ACLDefaultAdditiveBindTransform : rtm::qvv_identity();

			for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				Track[SampleIndex] = IdentityTransform;
			}
		}
		else
		{
			if (PhantomTrackMode == ACLPhantomTrackMode::Warn)
			{
				UE_LOGF(LogAnimationCompression, Warning, "Animation sequence has phantom tracks that do not map to any skeleton bone. Track %d in [%ls]", UETrackIndex, *CompressibleAnimData.FullName);
			}

			// We have raw track data, use it
			const FRawAnimSequenceTrack& RawTrack = RawTracks[UETrackIndex];

			for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				const FRawAnimTrackQuat& RotationSample = RawTrack.RotKeys.Num() == 1 ? RawTrack.RotKeys[0] : RawTrack.RotKeys[SampleIndex];
				Track[SampleIndex].rotation = UEQuatToACL(RotationSample);

				const FRawAnimTrackVector3& TranslationSample = RawTrack.PosKeys.Num() == 1 ? RawTrack.PosKeys[0] : RawTrack.PosKeys[SampleIndex];
				Track[SampleIndex].translation = UEVector3ToACL(TranslationSample);

				const FRawAnimTrackVector3& ScaleSample = RawTrack.ScaleKeys.Num() == 0 ? UEDefaultScale : (RawTrack.ScaleKeys.Num() == 1 ? RawTrack.ScaleKeys[0] : RawTrack.ScaleKeys[SampleIndex]);
				Track[SampleIndex].scale = UEVector3ToACL(ScaleSample);
			}
		}

		// Add our extra track
		Tracks[ACLTrackIndex] = MoveTemp(Track);
		ACLTrackIndex++;
	}

	// Number of UE tracks and ACL output tracks should match
	check(NumOutputACLTracks == NumUETracks);

	return Tracks;
}

uint32 GetNumSamples(const FCompressibleAnimData& CompressibleAnimData)
{
	return CompressibleAnimData.NumberOfKeys;
}

float GetSequenceLength(const UAnimSequence& AnimSeq)
{
	return AnimSeq.GetDataModel()->GetPlayLength();
}

namespace ACL
{
	namespace Private
	{
		float GetPerPlatformFloat(const FPerPlatformFloat& PerPlatformFloat, const ITargetPlatform* TargetPlatform)
		{
			if (TargetPlatform == nullptr)
			{
				// TODO: Why does calling GetDefault() not link with undefined symbol?
				return PerPlatformFloat.Default;	// Unknown target platform
			}

			return PerPlatformFloat.GetValueForPlatform(*TargetPlatform->IniPlatformName());
		}
	}
}
#endif	// WITH_EDITOR
