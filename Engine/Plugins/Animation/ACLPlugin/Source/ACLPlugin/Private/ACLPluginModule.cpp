// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "CoreMinimal.h"
#include "IACLPluginModule.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "ACLImpl.h"

// Enable console commands only in development builds when logging is enabled
#define WITH_ACL_CONSOLE_COMMANDS (!UE_BUILD_SHIPPING && !NO_LOGGING)

#if WITH_ACL_CONSOLE_COMMANDS
#include "AnimationCompressionLibraryDatabase.h"
#include "AnimBoneCompressionCodec_ACLDatabase.h"

#include "AnimationCompression.h"
#include "Animation/AnimBoneCompressionCodec.h"
#include "Animation/AnimBoneCompressionSettings.h"
#include "Animation/AnimCurveCompressionCodec.h"
#include "Animation/AnimCurveCompressionSettings.h"
#include "HAL/IConsoleManager.h"
#include "UObject/UObjectIterator.h"
#endif

#if WITH_EDITORONLY_DATA
#include "EditorDatabaseMonitor.h"
#endif

ACLAllocator ACLAllocatorImpl;

class FACLPlugin final : public IACLPlugin
{
private:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

#if WITH_ACL_CONSOLE_COMMANDS
	// Console commands
	void ListCodecs(const TArray<FString>& Args);
	void ListAnimSequences(const TArray<FString>& Args);
	void SetDatabaseVisualFidelity(const TArray<FString>& Args);

	TArray<IConsoleObject*> ConsoleCommands;
#endif

#if WITH_EDITORONLY_DATA
	void OnPostEngineInit();
#endif
};

IMPLEMENT_MODULE(FACLPlugin, ACLPlugin)

//////////////////////////////////////////////////////////////////////////

#if WITH_ACL_CONSOLE_COMMANDS
template<class ClassType>
static TArray<ClassType*> GetObjectInstancesSorted()
{
	TArray<ClassType*> Results;

	for (TObjectIterator<ClassType> It; It; ++It)
	{
		Results.Add(*It);
	}

	struct FCompareObjectNames
	{
		FORCEINLINE bool operator()(const ClassType& Lhs, const ClassType& Rhs) const
		{
			return Lhs.GetPathName().Compare(Rhs.GetPathName()) < 0;
		}
	};
	Results.Sort(FCompareObjectNames());

	return Results;
}

static double BytesToKB(SIZE_T NumBytes)
{
	return (double)NumBytes / 1024.0;
}

static double BytesToMB(SIZE_T NumBytes)
{
	return (double)NumBytes / (1024.0 * 1024.0);
}

template<typename SizeType>
static double Percentage(SizeType Part, SizeType Whole)
{
	return Whole != 0 ? (((double)Part / (double)Whole) * 100.0) : 0.0;
}

static SIZE_T GetCompressedBoneSize(const FCompressedAnimSequence& CompressedData)
{
	SIZE_T Size = CompressedData.CompressedTrackToSkeletonMapTable.GetAllocatedSize();
	if (CompressedData.CompressedDataStructure)
	{
		Size += CompressedData.CompressedDataStructure->GetApproxCompressedSize();
	}
	return Size;
}

static SIZE_T GetCompressedCurveSize(const FCompressedAnimSequence& CompressedData)
{
	SIZE_T Size = CompressedData.IndexedCurveNames.GetAllocatedSize();
	Size += CompressedData.CompressedCurveByteStream.GetAllocatedSize();
	return Size;
}

void FACLPlugin::ListCodecs(const TArray<FString>& Args)
{
	// Turn off log times to make diffing easier
	TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);

	// Make sure to log everything
	const ELogVerbosity::Type OldVerbosity = LogAnimationCompression.GetVerbosity();
	LogAnimationCompression.SetVerbosity(ELogVerbosity::All);

	const TArray<UAnimBoneCompressionSettings*> BoneSettings = GetObjectInstancesSorted<UAnimBoneCompressionSettings>();
	const TArray<UAnimBoneCompressionCodec*> BoneCodecs = GetObjectInstancesSorted<UAnimBoneCompressionCodec>();
	const TArray<UAnimCurveCompressionSettings*> CurveSettings = GetObjectInstancesSorted<UAnimCurveCompressionSettings>();
	const TArray<UAnimCurveCompressionCodec*> CurveCodecs = GetObjectInstancesSorted<UAnimCurveCompressionCodec>();
	const TArray<UAnimSequence*> AnimSequences = GetObjectInstancesSorted<UAnimSequence>();
	const TArray<UAnimationCompressionLibraryDatabase*> Databases = GetObjectInstancesSorted<UAnimationCompressionLibraryDatabase>();

	UE_LOGF(LogAnimationCompression, Log, "===== Bone Compression Setting Assets =====");
	for (const UAnimBoneCompressionSettings* Settings : BoneSettings)
	{
		int32 NumReferences = 0;
		SIZE_T TotalSize = 0;
		SIZE_T UsedSize = 0;
		for (const UAnimSequence* AnimSeq : AnimSequences)
		{
			UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSeq->GetCompressedData();
			const SIZE_T Size = GetCompressedBoneSize(CompressedAnimSequence.Get());
			if (AnimSeq->BoneCompressionSettings == Settings)
			{
				NumReferences++;
				UsedSize += Size;
			}
			TotalSize += Size;
		}

		UE_LOGF(LogAnimationCompression, Log, "%ls ...", *Settings->GetPathName());
		UE_LOGF(LogAnimationCompression, Log, "    used by %d / %d (%.1f %%) anim sequences", NumReferences, AnimSequences.Num(), Percentage(NumReferences, AnimSequences.Num()));
		UE_LOGF(LogAnimationCompression, Log, "    uses %.2f MB / %.2f MB (%.1f %%)", BytesToMB(UsedSize), BytesToMB(TotalSize), Percentage(UsedSize, TotalSize));
	}

	UE_LOGF(LogAnimationCompression, Log, "===== Bone Compression Codecs =====");
	for (const UAnimBoneCompressionCodec* Codec : BoneCodecs)
	{
		int32 NumReferences = 0;
		SIZE_T TotalSize = 0;
		SIZE_T UsedSize = 0;
		for (const UAnimSequence* AnimSeq : AnimSequences)
		{
			UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSeq->GetCompressedData();
			const SIZE_T Size = GetCompressedBoneSize(CompressedAnimSequence.Get());
			if (CompressedAnimSequence.Get().BoneCompressionCodec == Codec)
			{
				NumReferences++;
				UsedSize += Size;
			}
			TotalSize += Size;
		}

		if (Codec->Description.IsEmpty())
		{
			UE_LOGF(LogAnimationCompression, Log, "%ls ...", *Codec->GetPathName());
		}
		else
		{
			UE_LOGF(LogAnimationCompression, Log, "%ls (%ls) ...", *Codec->GetPathName(), *Codec->Description);
		}

		UE_LOGF(LogAnimationCompression, Log, "    used by %d / %d (%.1f %%) anim sequences", NumReferences, AnimSequences.Num(), Percentage(NumReferences, AnimSequences.Num()));
		UE_LOGF(LogAnimationCompression, Log, "    uses %.2f MB / %.2f MB (%.1f %%)", BytesToMB(UsedSize), BytesToMB(TotalSize), Percentage(UsedSize, TotalSize));
	}

	UE_LOGF(LogAnimationCompression, Log, "===== Curve Compression Setting Assets =====");
	for (const UAnimCurveCompressionSettings* Settings : CurveSettings)
	{
		int32 NumReferences = 0;
		SIZE_T TotalSize = 0;
		SIZE_T UsedSize = 0;
		for (const UAnimSequence* AnimSeq : AnimSequences)
		{
			UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSeq->GetCompressedData();
			const SIZE_T Size = GetCompressedCurveSize(CompressedAnimSequence.Get());
			if (AnimSeq->CurveCompressionSettings == Settings)
			{
				NumReferences++;
				UsedSize += Size;
			}
			TotalSize += Size;
		}

		UE_LOGF(LogAnimationCompression, Log, "%ls ...", *Settings->GetPathName());
		UE_LOGF(LogAnimationCompression, Log, "    used by %d / %d (%.1f %%) anim sequences", NumReferences, AnimSequences.Num(), Percentage(NumReferences, AnimSequences.Num()));
		UE_LOGF(LogAnimationCompression, Log, "    uses %.2f MB / %.2f MB (%.1f %%)", BytesToMB(UsedSize), BytesToMB(TotalSize), Percentage(UsedSize, TotalSize));
	}

	UE_LOGF(LogAnimationCompression, Log, "===== Curve Compression Codecs =====");
	for (const UAnimCurveCompressionCodec* Codec : CurveCodecs)
	{
		int32 NumReferences = 0;
		SIZE_T TotalSize = 0;
		SIZE_T UsedSize = 0;
		for (const UAnimSequence* AnimSeq : AnimSequences)
		{
			UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSeq->GetCompressedData();
			const SIZE_T Size = GetCompressedCurveSize(CompressedAnimSequence.Get());
			if (CompressedAnimSequence.Get().CurveCompressionCodec == Codec)
			{
				NumReferences++;
				UsedSize += Size;
			}
			TotalSize += Size;
		}

		UE_LOGF(LogAnimationCompression, Log, "%ls ...", *Codec->GetPathName());
		UE_LOGF(LogAnimationCompression, Log, "    used by %d / %d (%.1f %%) anim sequences", NumReferences, AnimSequences.Num(), Percentage(NumReferences, AnimSequences.Num()));
		UE_LOGF(LogAnimationCompression, Log, "    uses %.2f MB / %.2f MB (%.1f %%)", BytesToMB(UsedSize), BytesToMB(TotalSize), Percentage(UsedSize, TotalSize));
	}

	UE_LOGF(LogAnimationCompression, Log, "===== Animation Compression Library Database Assets =====");
	for (const UAnimationCompressionLibraryDatabase* Database : Databases)
	{
		int32 NumReferences = 0;
		for (const UAnimSequence* AnimSeq : AnimSequences)
		{
			UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSeq->GetCompressedData();
			UAnimBoneCompressionCodec_ACLDatabase* Codec = Cast<UAnimBoneCompressionCodec_ACLDatabase>(CompressedAnimSequence.Get().BoneCompressionCodec);
			if (Codec != nullptr && Codec->DatabaseAsset == Database)
			{
				NumReferences++;
			}
		}

		const acl::compressed_database* CompressedDatabase = acl::make_compressed_database(Database->CookedCompressedBytes.GetData());

		const uint32 DatabaseTotalSize = CompressedDatabase != nullptr ? CompressedDatabase->get_total_size() : 0;
		const uint32 DatabaseSize = CompressedDatabase != nullptr ? CompressedDatabase->get_size() : 0;
		const uint32 DatabaseBulkDataSizeMedium = CompressedDatabase != nullptr ? CompressedDatabase->get_bulk_data_size(acl::quality_tier::medium_importance) : 0;
		const uint32 DatabaseBulkDataSizeLow = CompressedDatabase != nullptr ? CompressedDatabase->get_bulk_data_size(acl::quality_tier::lowest_importance) : 0;
		const uint32 DatabaseBulkDataSize = DatabaseBulkDataSizeMedium + DatabaseBulkDataSizeLow;
		const uint32 SequencesSize = Database->CookedCompressedBytes.Num() - DatabaseSize;	// CompressedBytes contains the DB metadata and the sequences but not the bulk data

		UE_LOGF(LogAnimationCompression, Log, "%ls ...", *Database->GetPathName());
		UE_LOGF(LogAnimationCompression, Log, "    used by %d / %d (%.1f %%) anim sequences", NumReferences, AnimSequences.Num(), Percentage(NumReferences, AnimSequences.Num()));
		UE_LOGF(LogAnimationCompression, Log, "    sequences use %.2f MB", BytesToMB(SequencesSize));
		UE_LOGF(LogAnimationCompression, Log, "    database uses %.2f MB (%.2f MB streamable)", BytesToMB(DatabaseTotalSize), BytesToMB(DatabaseBulkDataSize));
	}

	LogAnimationCompression.SetVerbosity(OldVerbosity);
}

void FACLPlugin::ListAnimSequences(const TArray<FString>& Args)
{
	// Turn off log times to make diffing easier
	TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);

	// Make sure to log everything
	const ELogVerbosity::Type OldVerbosity = LogAnimationCompression.GetVerbosity();
	LogAnimationCompression.SetVerbosity(ELogVerbosity::All);

	const TArray<UAnimSequence*> AnimSequences = GetObjectInstancesSorted<UAnimSequence>();

	SIZE_T BoneDataTotalSize = 0;
	SIZE_T CurveDataTotalSize = 0;

	UE_LOGF(LogAnimationCompression, Log, "===== Anim Sequence Assets =====");
	for (const UAnimSequence* AnimSeq : AnimSequences)
	{
		UE_LOGF(LogAnimationCompression, Log, "%ls ...", *AnimSeq->GetPathName());

		UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSeq->GetCompressedData();
		const FCompressedAnimSequence& CompressedData = CompressedAnimSequence.Get();
		if (CompressedData.BoneCompressionCodec->Description.IsEmpty())
		{
			UE_LOGF(LogAnimationCompression, Log, "    uses bone codec %ls", *CompressedData.BoneCompressionCodec->GetPathName());
		}
		else
		{
			UE_LOGF(LogAnimationCompression, Log, "    uses bone codec %ls (%ls)", *CompressedData.BoneCompressionCodec->GetPathName(), *CompressedData.BoneCompressionCodec->Description);
		}

		const SIZE_T BoneDataSize = GetCompressedBoneSize(CompressedData);
		UE_LOGF(LogAnimationCompression, Log, "    has %.2f KB of bone data", BytesToKB(BoneDataSize));

#if WITH_EDITORONLY_DATA
		if (CompressedData.CompressedDataStructure)
		{
			UE_LOGF(LogAnimationCompression, Log, "    has a bone error of %.4f cm", CompressedData.CompressedDataStructure->BoneCompressionErrorStats.MaxError);
		}
#endif

		UE_LOGF(LogAnimationCompression, Log, "    uses curve codec %ls", *CompressedData.CurveCompressionCodec->GetPathName());

		const SIZE_T CurveDataSize = GetCompressedCurveSize(CompressedData);
		UE_LOGF(LogAnimationCompression, Log, "    has %.2f KB of curve data", BytesToKB(CurveDataSize));

		BoneDataTotalSize += BoneDataSize;
		CurveDataTotalSize += CurveDataSize;
	}

	UE_LOGF(LogAnimationCompression, Log, "Total bone data size: %.2f MB", BytesToMB(BoneDataTotalSize));
	UE_LOGF(LogAnimationCompression, Log, "Total curve data size: %.2f MB", BytesToMB(CurveDataTotalSize));

	LogAnimationCompression.SetVerbosity(OldVerbosity);
}

void FACLPlugin::SetDatabaseVisualFidelity(const TArray<FString>& Args)
{
	// Make sure to log everything
	const ELogVerbosity::Type OldVerbosity = LogAnimationCompression.GetVerbosity();
	LogAnimationCompression.SetVerbosity(ELogVerbosity::All);

	ACLVisualFidelity Fidelity = ACLVisualFidelity::Highest;
	if (Args.Contains(TEXT("Highest")))
	{
		Fidelity = ACLVisualFidelity::Highest;
	}
	else if (Args.Contains(TEXT("Medium")))
	{
		Fidelity = ACLVisualFidelity::Medium;
	}
	else if (Args.Contains(TEXT("Lowest")))
	{
		Fidelity = ACLVisualFidelity::Lowest;
	}
	else if (Args.Num() != 0)
	{
		UE_LOGF(LogAnimationCompression, Warning, "Invalid visual fidelity: %ls", *Args[0]);
	}

	const TArray<UAnimationCompressionLibraryDatabase*> DatabaseAssets = GetObjectInstancesSorted<UAnimationCompressionLibraryDatabase>();
	for (UAnimationCompressionLibraryDatabase* DatabaseAsset : DatabaseAssets)
	{
		DatabaseAsset->SetVisualFidelity(Fidelity);
	}

	LogAnimationCompression.SetVerbosity(OldVerbosity);
}
#endif

#if WITH_EDITORONLY_DATA
void FACLPlugin::OnPostEngineInit()
{
	EditorDatabaseMonitor::RegisterMonitor();
}
#endif

void FACLPlugin::StartupModule()
{
#if WITH_ACL_CONSOLE_COMMANDS
	if (!IsRunningCommandlet())
	{
		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("ACL.ListCodecs"),
			TEXT("Dumps statistics about animation codecs to the log."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FACLPlugin::ListCodecs),
			ECVF_Default
		));

		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("ACL.ListAnimSequences"),
			TEXT("Dumps statistics about animation sequences to the log."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FACLPlugin::ListAnimSequences),
			ECVF_Default
		));

		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("ACL.SetDatabaseVisualFidelity"),
			TEXT("Sets the visual fidelity of all ACL databases. Argument: Highest (default if no argument is provided), Medium, Lowest"),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FACLPlugin::SetDatabaseVisualFidelity),
			ECVF_Default
		));
	}
#endif

#if WITH_EDITORONLY_DATA
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FACLPlugin::OnPostEngineInit);
#endif
}

void FACLPlugin::ShutdownModule()
{
#if WITH_EDITORONLY_DATA
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	EditorDatabaseMonitor::UnregisterMonitor();
#endif

#if WITH_ACL_CONSOLE_COMMANDS
	for (IConsoleObject* Cmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}
	ConsoleCommands.Empty();
#endif
}
