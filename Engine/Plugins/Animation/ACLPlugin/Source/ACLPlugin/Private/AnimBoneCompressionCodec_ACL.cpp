// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "AnimBoneCompressionCodec_ACL.h"

#if WITH_EDITORONLY_DATA
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"

#include "ACLImpl.h"

THIRD_PARTY_INCLUDES_START
#include <acl/compression/track_error.h>
#include <acl/core/bitset.h>
#include <acl/decompression/decompress.h>
THIRD_PARTY_INCLUDES_END

#endif	// WITH_EDITORONLY_DATA

#include "ACLDecompressionImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBoneCompressionCodec_ACL)

UAnimBoneCompressionCodec_ACL::UAnimBoneCompressionCodec_ACL(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, KeyframeStrippingProportion(0.0f)		// Strip nothing by default since it is destructive
	, KeyframeStrippingThreshold(0.0f)		// Strip nothing by default since it is destructive
#endif
{
}

#if WITH_EDITORONLY_DATA
void UAnimBoneCompressionCodec_ACL::GetCompressionSettings(const class ITargetPlatform* TargetPlatform, acl::compression_settings& OutSettings) const
{
	OutSettings = acl::get_default_compression_settings();

	OutSettings.level = GetCompressionLevel(CompressionLevel);

	OutSettings.keyframe_stripping.proportion = ACL::Private::GetPerPlatformFloat(KeyframeStrippingProportion, TargetPlatform);
	OutSettings.keyframe_stripping.threshold = ACL::Private::GetPerPlatformFloat(KeyframeStrippingThreshold, TargetPlatform);
}

void UAnimBoneCompressionCodec_ACL::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	Super::PopulateDDCKey(KeyArgs, Ar);

	acl::compression_settings Settings;
	GetCompressionSettings(KeyArgs.TargetPlatform, Settings);

	uint32 ForceRebuildVersion = 1;
	uint32 SettingsHash = Settings.get_hash();

	Ar << ForceRebuildVersion << SettingsHash;

	for (USkeletalMesh* SkelMesh : OptimizationTargets)
	{
		FSkeletalMeshModel* MeshModel = SkelMesh != nullptr ? SkelMesh->GetImportedModel() : nullptr;
		if (MeshModel != nullptr)
		{
			Ar << MeshModel->SkeletalMeshModelGUID;
		}
	}
}
#endif // WITH_EDITORONLY_DATA

void UAnimBoneCompressionCodec_ACL::DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs, TArrayView<FTransform>& OutAtoms) const
{
	const FACLCompressedAnimData& AnimData = static_cast<const FACLCompressedAnimData&>(DecompContext.CompressedAnimData);
	const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
	check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

	acl::decompression_context<UEDefaultDecompressionSettings> ACLContext;
	ACLContext.initialize(*CompressedClipData);

	::DecompressPose(DecompContext, ACLContext, RotationPairs, TranslationPairs, ScalePairs, OutAtoms);
}

void UAnimBoneCompressionCodec_ACL::DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const UE::Anim::FAnimPoseDecompressionData& DecompressionData) const
{
	const FACLCompressedAnimData& AnimData = static_cast<const FACLCompressedAnimData&>(DecompContext.CompressedAnimData);
	const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
	check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

	acl::decompression_context<UEDefaultDecompressionSettings> ACLContext;
	ACLContext.initialize(*CompressedClipData);

	::DecompressPose(DecompContext, ACLContext, DecompressionData);
}

void UAnimBoneCompressionCodec_ACL::DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const
{
	const FACLCompressedAnimData& AnimData = static_cast<const FACLCompressedAnimData&>(DecompContext.CompressedAnimData);
	const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
	check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

	acl::decompression_context<UEDefaultDecompressionSettings> ACLContext;
	ACLContext.initialize(*CompressedClipData);

	::DecompressBone(DecompContext, ACLContext, TrackIndex, OutAtom);
}
