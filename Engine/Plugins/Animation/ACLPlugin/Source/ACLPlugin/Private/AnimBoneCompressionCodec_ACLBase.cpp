// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "AnimBoneCompressionCodec_ACLBase.h"
#include "Animation/Skeleton.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBoneCompressionCodec_ACLBase)

#if WITH_EDITORONLY_DATA
#include "AnimBoneCompressionCodec_ACLSafe.h"
#include "Animation/AnimationSettings.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Runtime/Launch/Resources/Version.h"

#include "ACLImpl.h"

THIRD_PARTY_INCLUDES_START
#include <acl/compression/compress.h>
#include <acl/compression/pre_process.h>
#include <acl/compression/transform_error_metrics.h>
#include <acl/compression/track_error.h>
#include <acl/core/bitset.h>
#include <acl/core/compressed_tracks_version.h>
#include <acl/decompression/decompress.h>
THIRD_PARTY_INCLUDES_END
#endif	// WITH_EDITORONLY_DATA

#include <acl/core/compressed_tracks.h>

void FACLCompressedAnimDataBase::SerializeCompressedData(UObject* DataOwner, FArchive& Ar)
{
	ICompressedAnimData::SerializeCompressedData(DataOwner, Ar);

	Ar << bCompressionFailed;

	if (bCompressionFailed && (Ar.IsLoading() || Ar.IsCooking()))
	{
		// Compression failed due to invalid settings or data (see below in Compress(..))
		// We'll end up outputting the bind pose during decompression
		// We report an error to cause the cook to fail
		UE_LOGF(LogAnimationCompression, Error,
			"ACL failed to compress an anim sequence and will output the bind pose at runtime: %ls",
			DataOwner != nullptr ? *DataOwner->GetPathName() : TEXT("[Unknown Sequence]"));
	}
}

bool FACLCompressedAnimData::IsValid() const
{
	if (CompressedByteStream.Num() == 0)
	{
		return false;
	}

	const acl::compressed_tracks* CompressedClipData = acl::make_compressed_tracks(CompressedByteStream.GetData());
	return CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty();
}

UAnimBoneCompressionCodec_ACLBase::UAnimBoneCompressionCodec_ACLBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	CompressionLevel = ACLCL_Automatic;
	PhantomTrackMode = ACLPhantomTrackMode::Ignore;	// Same as UE codecs

	// We use a higher virtual vertex distance when bones have a socket attached or are keyed end effectors (IK, hand, camera, etc)
	// We use 100cm instead of 3cm. UE 4 usually uses 50cm (END_EFFECTOR_DUMMY_BONE_LENGTH_SOCKET) but
	// we use a higher value anyway due to the fact that ACL has no error compensation and it is more aggressive.
	DefaultVirtualVertexDistance = 3.0f;	// 3cm, suitable for ordinary characters
	SafeVirtualVertexDistance = 100.0f;		// 100cm

	ErrorThreshold = 0.01f;					// 0.01cm, conservative enough for cinematographic quality
#endif	// WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
bool UAnimBoneCompressionCodec_ACLBase::IsHighFidelity(const FCompressibleAnimData& CompressibleAnimData) const
{
	// ACL handles raw data sanitizing internally
	return true;
}

static void AppendMaxVertexDistances(USkeletalMesh* OptimizationTarget, TMap<FName, float>& BoneMaxVertexDistanceMap)
{
#if (ENGINE_MAJOR_VERSION == 4 && ENGINE_MINOR_VERSION >= 27) || ENGINE_MAJOR_VERSION >= 5
	USkeleton* Skeleton = OptimizationTarget != nullptr ? OptimizationTarget->GetSkeleton() : nullptr;
#else
	USkeleton* Skeleton = OptimizationTarget != nullptr ? OptimizationTarget->Skeleton : nullptr;
#endif

	if (Skeleton == nullptr)
	{
		return; // No data to work with
	}

	const FSkeletalMeshModel* MeshModel = OptimizationTarget->GetImportedModel();
	if (MeshModel == nullptr || MeshModel->LODModels.Num() == 0)
	{
		return;	// No data to work with
	}

	const FReferenceSkeleton& RefSkeleton = Skeleton->GetReferenceSkeleton();
	const TArray<FTransform>& RefSkeletonPose = RefSkeleton.GetRefBonePose();
	const uint32 NumBones = RefSkeletonPose.Num();

	TArray<FTransform> RefSkeletonObjectSpacePose;
	RefSkeletonObjectSpacePose.AddUninitialized(NumBones);
	for (uint32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const int32 ParentBoneIndex = RefSkeleton.GetParentIndex(BoneIndex);
		if (ParentBoneIndex != INDEX_NONE)
		{
			RefSkeletonObjectSpacePose[BoneIndex] = RefSkeletonPose[BoneIndex] * RefSkeletonObjectSpacePose[ParentBoneIndex];
		}
		else
		{
			RefSkeletonObjectSpacePose[BoneIndex] = RefSkeletonPose[BoneIndex];
		}
	}

	// Iterate over every vertex and track which one is the most distant for every bone
	TArray<float> MostDistantVertexDistancePerBone;
	MostDistantVertexDistancePerBone.AddZeroed(NumBones);

	const uint32 NumSections = MeshModel->LODModels[0].Sections.Num();
	for (uint32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		const FSkelMeshSection& Section = MeshModel->LODModels[0].Sections[SectionIndex];
		const uint32 NumVertices = Section.SoftVertices.Num();

		for (uint32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
		{
			const FSoftSkinVertex& VertexInfo = Section.SoftVertices[VertexIndex];
			const FVector VertexPosition = UEVector3Cast(VertexInfo.Position);

			for (uint32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
			{
				if (VertexInfo.InfluenceWeights[InfluenceIndex] != 0)
				{
					const uint32 SectionBoneIndex = VertexInfo.InfluenceBones[InfluenceIndex];
					const uint32 BoneIndex = Section.BoneMap[SectionBoneIndex];

					const FTransform& BoneTransform = RefSkeletonObjectSpacePose[BoneIndex];
					const FVector BoneTranslation = UEVector3Cast(BoneTransform.GetTranslation());

					const float VertexDistanceToBone = FVector::Distance(VertexPosition, BoneTranslation);

					float& MostDistantVertexDistance = MostDistantVertexDistancePerBone[BoneIndex];
					MostDistantVertexDistance = FMath::Max(MostDistantVertexDistance, VertexDistanceToBone);
				}
			}
		}
	}

	// Store the results in a map by bone name since the optimizing target might use a different
	// skeleton mapping.
	for (uint32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
		const float MostDistantVertexDistance = MostDistantVertexDistancePerBone[BoneIndex];

		float& BoneMaxVertexDistance = BoneMaxVertexDistanceMap.FindOrAdd(BoneName, 0.0f);
		BoneMaxVertexDistance = FMath::Max(BoneMaxVertexDistance, MostDistantVertexDistance);
	}
}

static void PopulateShellDistanceFromOptimizationTargets(const FCompressibleAnimData& CompressibleAnimData, const TArray<USkeletalMesh*>& OptimizationTargets, acl::track_array_qvvf& ACLTracks)
{
	// For each bone, get the furtest vertex distance
	TMap<FName, float> BoneMaxVertexDistanceMap;
	for (USkeletalMesh* OptimizationTarget : OptimizationTargets)
	{
		AppendMaxVertexDistances(OptimizationTarget, BoneMaxVertexDistanceMap);
	}

	const uint32 NumBones = ACLTracks.get_num_tracks();
	for (uint32 ACLBoneIndex = 0; ACLBoneIndex < NumBones; ++ACLBoneIndex)
	{
		acl::track_qvvf& ACLTrack = ACLTracks[ACLBoneIndex];
		const FName BoneName(ACLTrack.get_name().c_str());

		const float* MostDistantVertexDistance = BoneMaxVertexDistanceMap.Find(BoneName);
		if (MostDistantVertexDistance == nullptr || *MostDistantVertexDistance <= 0.0F)
		{
			continue;	// No skinned vertices for this bone, skipping
		}

		const FBoneData& UEBone = CompressibleAnimData.BoneData[ACLBoneIndex];

		acl::track_desc_transformf& Desc = ACLTrack.get_description();

		// We set our shell distance to the most distant vertex distance.
		// This ensures that we measure the error where that vertex lies.
		// Together with the precision value, all vertices skinned to this bone
		// will be guaranteed to have an error smaller or equal to the precision
		// threshold used.
		if (UEBone.bHasSocket || UEBone.bKeyEndEffector)
		{
			// Bones that have sockets or are key end effectors require extra precision, make sure
			// that our shell distance is at least what we ask of it regardless of the skinning
			// information.
			Desc.shell_distance = FMath::Max(Desc.shell_distance, *MostDistantVertexDistance);
		}
		else
		{
			// This could be higher or lower than the default value used by ordinary bones.
			// This thus taylors the shell distance to the visual mesh.
			Desc.shell_distance = *MostDistantVertexDistance;
		}
	}
}

static void StripBindPose(const FCompressibleAnimData& CompressibleAnimData, acl::track_array_qvvf& ACLTracks)
{
	// Additive sequences use the identity as their bind pose, no need for stripping
	check(!CompressibleAnimData.bIsValidAdditive);

	const int32 NumBones = CompressibleAnimData.BoneData.Num();

	for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const FBoneData& UEBone = CompressibleAnimData.BoneData[BoneIndex];

		acl::track_qvvf& Track = ACLTracks[BoneIndex];
		acl::track_desc_transformf& Desc = Track.get_description();

		// When we decompress a whole pose, the output buffer will already contain the bind pose.
		// As such, we skip all default sub-tracks and avoid writing anything to the output pose.
		// By setting the default value to the bind pose, default sub-tracks will be equal to the bind pose
		// and be stripped from the compressed data buffer entirely.

		// As such, here are the potential behaviors for non-animated bones equal to the default_value below:
		//     A stripped bone equal to the bind pose (stripped)
		//         Skipped during whole pose decompression, already present in output buffer
		//         Single bone decompression will output the bind pose taken from the decompression context
		//     A stripped bone not equal to the bind pose (it won't be stripped nor skipped)
		//         Decompressed normally with the rest of the pose and written to the output buffer
		//         Single bone decompression will output the correct value

		// Set the default value to the bind pose so that it can be stripped
		Desc.default_value = rtm::qvv_set(UEQuatToACL(UEBone.Orientation), UEVector3ToACL(UEBone.Position), UEVector3ToACL(UEBone.Scale));
	}
}

static void ResetTracksToIdentity(const FCompressibleAnimData& CompressibleAnimData, bool bBuildAdditiveBase, acl::track_array_qvvf& ACLTracks)
{
	// This resets the input ACL tracks to the identity transform but retains all other values
	const uint32 NumSamples = 1;
	const float SampleRate = 30.0f;

	// Additive animations have 0,0,0 scale as the default since we add it
	const bool bIsAdditive = bBuildAdditiveBase ? false : CompressibleAnimData.bIsValidAdditive;
	const FVector3f UE4DefaultScale(bIsAdditive ? 0.0f : 1.0f);
	const rtm::vector4f ACLDefaultScale = rtm::vector_set(bIsAdditive ? 0.0f : 1.0f);

	rtm::qvvf ACLIdentityTransform = rtm::qvv_identity();
	ACLIdentityTransform.scale = ACLDefaultScale;

	const acl::track_desc_transformf DefaultDesc;

	for (acl::track_qvvf& ACLTrack : ACLTracks)
	{
		// Reset everything to the identity transform and default values
		// Retain the output index to ensure proper output size
		acl::track_desc_transformf Desc = ACLTrack.get_description();	// Copy
		Desc.default_value = ACLIdentityTransform;
		Desc.precision = DefaultDesc.precision;
		Desc.shell_distance = DefaultDesc.shell_distance;
		Desc.parent_index = acl::k_invalid_track_index;

		// Reset track to a single sample
		ACLTrack = acl::track_qvvf::make_reserve(Desc, ACLAllocatorImpl, NumSamples, SampleRate);
		ACLTrack[0] = ACLIdentityTransform;
	}
}

bool UAnimBoneCompressionCodec_ACLBase::Compress(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult)
{
	acl::track_array_qvvf ACLTracks = BuildACLTransformTrackArray(ACLAllocatorImpl, CompressibleAnimData, DefaultVirtualVertexDistance, SafeVirtualVertexDistance, false, PhantomTrackMode);

	acl::track_array_qvvf ACLBaseTracks;
	if (CompressibleAnimData.bIsValidAdditive)
	{
		ACLBaseTracks = BuildACLTransformTrackArray(ACLAllocatorImpl, CompressibleAnimData, DefaultVirtualVertexDistance, SafeVirtualVertexDistance, true, PhantomTrackMode);
	}

	UE_LOGF(LogAnimationCompression, Verbose, "ACL Animation raw size: %u bytes [%ls]", ACLTracks.get_raw_size(), *CompressibleAnimData.FullName);

	// If we have an optimization target, use it
	TArray<USkeletalMesh*> OptimizationTargets = GetOptimizationTargets();
	if (OptimizationTargets.Num() != 0)
	{
		PopulateShellDistanceFromOptimizationTargets(CompressibleAnimData, OptimizationTargets, ACLTracks);
	}

	// Set our error threshold
	for (acl::track_qvvf& Track : ACLTracks)
	{
		Track.get_description().precision = ErrorThreshold;
	}

	// Enable bind pose stripping if we need to.
	// Additive sequences have their bind pose equivalent as the additive identity transform and as
	// such, ACL performs stripping by default and everything works great.
	// See [Bind pose stripping] for details
	const bool bUsesBindPoseStripping = !CompressibleAnimData.bIsValidAdditive;
	if (bUsesBindPoseStripping)
	{
		StripBindPose(CompressibleAnimData, ACLTracks);
	}

	acl::compression_settings Settings;
	GetCompressionSettings(CompressibleAnimData.TargetPlatform, Settings);

	constexpr acl::additive_clip_format8 AdditiveFormat = acl::additive_clip_format8::additive1;

	acl::qvvf_transform_error_metric DefaultErrorMetric;
	acl::additive_qvvf_transform_error_metric<AdditiveFormat> AdditiveErrorMetric;
	if (!ACLBaseTracks.is_empty())
	{
		Settings.error_metric = &AdditiveErrorMetric;
	}
	else
	{
		Settings.error_metric = &DefaultErrorMetric;
	}

	{
		// We pre-process the raw tracks to prime them for compression
		acl::pre_process_settings_t PreProcessSettings;
		PreProcessSettings.actions = acl::pre_process_actions::recommended;

		// If we retain full precision, use lossless pre-processing
		if (Settings.rotation_format == acl::rotation_format8::quatf_full ||
			Settings.rotation_format == acl::rotation_format8::quatf_drop_w_full ||
			Settings.translation_format == acl::vector_format8::vector3f_full ||
			Settings.scale_format == acl::vector_format8::vector3f_full)
		{
			PreProcessSettings.precision_policy = acl::pre_process_precision_policy::lossless;
		}
		else
		{
			PreProcessSettings.precision_policy = acl::pre_process_precision_policy::lossy;
		}

		PreProcessSettings.error_metric = Settings.error_metric;

		if (!ACLBaseTracks.is_empty())
		{
			PreProcessSettings.additive_base = &ACLBaseTracks;
			PreProcessSettings.additive_format = AdditiveFormat;
		}

		acl::pre_process_track_list(ACLAllocatorImpl, PreProcessSettings, ACLTracks);
	}

	acl::output_stats Stats;
	acl::compressed_tracks* CompressedTracks = nullptr;
	acl::error_result CompressionResult = acl::compress_track_list(ACLAllocatorImpl, ACLTracks, Settings, ACLBaseTracks, AdditiveFormat, CompressedTracks, Stats);

	bool bEnableErrorReporting = true;
	bool bCompressionFailed = false;

	if (!CompressionResult.empty())
	{
		// If compression failed, one of two things happened:
		//    * Invalid settings were used, this would be a code/logic error that results in an improper usage of ACL
		//    * Invalid data was provided, this would be a validation error that should ideally be caught earlier (e.g import, save)
		// 
		// Either way, if we get here, we cannot recover and we cannot fail as the engine assumes that compression always succeeds.
		// We must handle failure gracefully. To that end, we compress an empty stub to ensure that something is present to
		// decompress. Because the stub is empty, we'll simply output the bind pose. We still log this as an error to signal that
		// this is a problem that needs to be fixed. This will allow the editor to continue working with the bind pose we'll output
		// but cooking will fail preventing us from running with invalid state.

		UE_LOGF(LogAnimationCompression, Error, "ACL failed to compress anim sequence: %ls [%ls]", ANSI_TO_TCHAR(CompressionResult.c_str()), *CompressibleAnimData.FullName);

		// We reset the tracks to the identity, getting rid of any potentially invalid data.
		// By setting them to the identity along with their default value as well, bind pose stripping will
		// strip the single keyframe. This will result in the bind pose being outputted during decompression
		// for non-additive animations and additive animations will retain the additive identity.
		ResetTracksToIdentity(CompressibleAnimData, false, ACLTracks);
		if (CompressibleAnimData.bIsValidAdditive)
		{
			ResetTracksToIdentity(CompressibleAnimData, true, ACLBaseTracks);
		}

		CompressionResult = acl::compress_track_list(ACLAllocatorImpl, ACLTracks, Settings, ACLBaseTracks, AdditiveFormat, CompressedTracks, Stats);

		// The stub compression should never fail
		check(CompressionResult.empty() && CompressedTracks != nullptr);

		// Because we compress an empty stub, disable error reporting below
		bEnableErrorReporting = false;
		bCompressionFailed = true;
	}

	checkSlow(CompressedTracks->is_valid(true).empty());

	const uint32 CompressedClipDataSize = CompressedTracks->get_size();

	OutResult.CompressedByteStream.Empty(CompressedClipDataSize);
	OutResult.CompressedByteStream.AddUninitialized(CompressedClipDataSize);
	FMemory::Memcpy(OutResult.CompressedByteStream.GetData(), CompressedTracks, CompressedClipDataSize);

	OutResult.Codec = this;

	OutResult.AnimData = AllocateAnimData();

	OutResult.AnimData->CompressedNumberOfKeys = GetNumSamples(CompressibleAnimData);

	FACLCompressedAnimDataBase& AnimData = static_cast<FACLCompressedAnimDataBase&>(*OutResult.AnimData);
	AnimData.bCompressionFailed = bCompressionFailed;

#if !NO_LOGGING
	if (bEnableErrorReporting && LogAnimationCompression.GetVerbosity() >= ELogVerbosity::Verbose)
	{
		// Use debug settings in case codec picked is the fallback
		acl::decompression_context<UEDebugDecompressionSettings> Context;
		Context.initialize(*CompressedTracks);

		const acl::track_error TrackError = acl::calculate_compression_error(ACLAllocatorImpl, ACLTracks, Context, *Settings.error_metric, ACLBaseTracks);

		UE_LOGF(LogAnimationCompression, Verbose, "ACL Animation compressed size: %u bytes [%ls]", CompressedClipDataSize, *CompressibleAnimData.FullName);
		UE_LOGF(LogAnimationCompression, Verbose, "ACL Animation error: %.4f cm (bone %u @ %.3f) [%ls]", TrackError.error, TrackError.index, TrackError.sample_time, *CompressibleAnimData.FullName);
	}
#endif

	ACLAllocatorImpl.deallocate(CompressedTracks, CompressedClipDataSize);

	// Allow codecs to override final anim data and result
	PostCompression(CompressibleAnimData, OutResult);

	// Bind our compressed sequence data buffer
	OutResult.AnimData->Bind(OutResult.CompressedByteStream);

	return true;
}

// TODO: Use CompressibleAnimData::RefLocalPoses for bind pose instance of BoneData

void UAnimBoneCompressionCodec_ACLBase::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	Super::PopulateDDCKey(KeyArgs, Ar);

	uint32 ForceRebuildVersion = 20;

	Ar << ForceRebuildVersion << DefaultVirtualVertexDistance << SafeVirtualVertexDistance << ErrorThreshold;
	Ar << CompressionLevel;
	Ar << PhantomTrackMode;

	uint16 LatestACLVersion = static_cast<uint16>(acl::compressed_tracks_version16::latest);
	Ar << LatestACLVersion;

	// Add the end effector match name list since if it changes, we need to re-compress
	const TArray<FString>& KeyEndEffectorsMatchNameArray = UAnimationSettings::Get()->KeyEndEffectorsMatchNameArray;
	for (const FString& MatchName : KeyEndEffectorsMatchNameArray)
	{
		uint32 MatchNameHash = GetTypeHash(MatchName);
		Ar << MatchNameHash;
	}

	// Additive sequences use the additive identity as their bind pose, no need for stripping
	if (!KeyArgs.AnimSequence.IsValidAdditive())
	{
		// When bind pose stripping is enabled, we have to include the bind pose in the DDC key.
		// If a sequence is compressed with bind pose A, and we strip a few bones and later modify the bind pose,
		// bind pose B might now contain values that would not be stripped in our sequence.
		// To avoid data being stale, the DDC must reflect this.

		// TODO: It would be nice if Epic provided a GUID for the bind pose uniqueness to speed this up

		const USkeleton* Skeleton = KeyArgs.AnimSequence.GetSkeleton();
		const TArray<FTransform>& BindPose = Skeleton->GetRefLocalPoses();
		for (const FTransform& BoneBindTransform : BindPose)
		{
			FQuat Rotation = BoneBindTransform.GetRotation();
			Ar << Rotation;

			FVector Translation = BoneBindTransform.GetTranslation();
			Ar << Translation;

			FVector Scale = BoneBindTransform.GetScale3D();
			Ar << Scale;
		}
	}
}

int64 UAnimBoneCompressionCodec_ACLBase::EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const
{
	const int64 AnimSeqRawSize = AnimSequence.GetApproxBoneRawSize();

	int64 EstimatedMemoryUsage = 0;
	EstimatedMemoryUsage += AnimSeqRawSize;	// We copy the raw data into the ACL format

	if (AnimSequence.IsValidAdditive())
	{
		if (AnimSequence.RefPoseSeq)
		{
			// We copy the additive base into the ACL format
			EstimatedMemoryUsage += AnimSequence.RefPoseSeq->GetApproxBoneRawSize();
		}
		else
		{
			// We create the additive base in the ACL format, we use the same estimate as FAnimationSequenceAsyncCacheTask::GetRequiredMemoryEstimate()
			EstimatedMemoryUsage += AnimSeqRawSize;
		}
	}

	EstimatedMemoryUsage *= 2;	// Internally, ACL copies the raw data into a different format than the input because the input is not modified

	EstimatedMemoryUsage += AnimSeqRawSize;	// ACL keeps a mutable copy of the lossy data that it modifies during compression
	EstimatedMemoryUsage += AnimSeqRawSize;	// ACL will allocate the output buffer, assume that it's as large as the raw data
	EstimatedMemoryUsage += 100 * 1024;		// Reserve 100 KB for internal bookkeeping and other required metadata

	return EstimatedMemoryUsage;
}
#endif

TUniquePtr<ICompressedAnimData> UAnimBoneCompressionCodec_ACLBase::AllocateAnimData() const
{
	return MakeUnique<FACLCompressedAnimData>();
}

void UAnimBoneCompressionCodec_ACLBase::ByteSwapIn(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryReader& MemoryStream) const
{
#if !PLATFORM_LITTLE_ENDIAN
#error "ACL does not currently support big-endian platforms"
#endif

	// ByteSwapIn(..) is called on load

	// TODO: ACL does not support byte swapping
	MemoryStream.Serialize(CompressedData.GetData(), CompressedData.Num());
}

void UAnimBoneCompressionCodec_ACLBase::ByteSwapOut(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryWriter& MemoryStream) const
{
#if !PLATFORM_LITTLE_ENDIAN
#error "ACL does not currently support big-endian platforms"
#endif

	// ByteSwapOut(..) is called on save, during cooking, or when counting memory

	// TODO: ACL does not support byte swapping
	MemoryStream.Serialize(CompressedData.GetData(), CompressedData.Num());
}
