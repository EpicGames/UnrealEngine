// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "AnimBoneCompressionCodec_ACLCustom.h"

#include "ACLDecompressionImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBoneCompressionCodec_ACLCustom)

#if WITH_EDITORONLY_DATA
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"

THIRD_PARTY_INCLUDES_START
#include <acl/compression/compression_settings.h>
THIRD_PARTY_INCLUDES_END
#endif

UAnimBoneCompressionCodec_ACLCustom::UAnimBoneCompressionCodec_ACLCustom(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, RotationFormat(ACLRotationFormat::ACLRF_QuatDropW_Variable)
	, TranslationFormat(ACLVectorFormat::ACLVF_Vector3_Variable)
	, ScaleFormat(ACLVectorFormat::ACLVF_Vector3_Variable)
	, KeyframeStrippingProportion(0.0f)		// Strip nothing by default since it is destructive
	, KeyframeStrippingThreshold(0.0f)		// Strip nothing by default since it is destructive
#endif
{
}

#if WITH_EDITORONLY_DATA
void UAnimBoneCompressionCodec_ACLCustom::GetCompressionSettings(const class ITargetPlatform* TargetPlatform, acl::compression_settings& OutSettings) const
{
	OutSettings = acl::compression_settings();

	OutSettings.rotation_format = GetRotationFormat(RotationFormat);
	OutSettings.translation_format = GetVectorFormat(TranslationFormat);
	OutSettings.scale_format = GetVectorFormat(ScaleFormat);

	OutSettings.level = GetCompressionLevel(CompressionLevel);

	OutSettings.keyframe_stripping.proportion = ACL::Private::GetPerPlatformFloat(KeyframeStrippingProportion, TargetPlatform);
	OutSettings.keyframe_stripping.threshold = ACL::Private::GetPerPlatformFloat(KeyframeStrippingThreshold, TargetPlatform);
}

void UAnimBoneCompressionCodec_ACLCustom::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	Super::PopulateDDCKey(KeyArgs, Ar);

	acl::compression_settings Settings;
	GetCompressionSettings(KeyArgs.TargetPlatform, Settings);

	uint32 ForceRebuildVersion = 1;
	uint32 SettingsHash = Settings.get_hash();

	Ar	<< ForceRebuildVersion << SettingsHash;

	for (USkeletalMesh* SkelMesh : OptimizationTargets)
	{
		FSkeletalMeshModel* MeshModel = SkelMesh != nullptr ? SkelMesh->GetImportedModel() : nullptr;
		if (MeshModel != nullptr)
		{
			Ar << MeshModel->SkeletalMeshModelGUID;
		}
	}
}
#endif // WITH_EDITORONLY_DATA

void UAnimBoneCompressionCodec_ACLCustom::DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs, TArrayView<FTransform>& OutAtoms) const
{
	const FACLCompressedAnimData& AnimData = static_cast<const FACLCompressedAnimData&>(DecompContext.CompressedAnimData);
	const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
	check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

	acl::decompression_context<UECustomDecompressionSettings> ACLContext;
	ACLContext.initialize(*CompressedClipData);

	::DecompressPose(DecompContext, ACLContext, RotationPairs, TranslationPairs, ScalePairs, OutAtoms);
}

void UAnimBoneCompressionCodec_ACLCustom::DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const
{
	const FACLCompressedAnimData& AnimData = static_cast<const FACLCompressedAnimData&>(DecompContext.CompressedAnimData);
	const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
	check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

	acl::decompression_context<UECustomDecompressionSettings> ACLContext;
	ACLContext.initialize(*CompressedClipData);

	::DecompressBone(DecompContext, ACLContext, TrackIndex, OutAtom);
}
