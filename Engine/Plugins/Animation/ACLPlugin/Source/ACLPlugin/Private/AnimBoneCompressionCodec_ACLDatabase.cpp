// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "AnimBoneCompressionCodec_ACLDatabase.h"

#include "Algo/BinarySearch.h"

#if WITH_EDITORONLY_DATA
#include "Animation/AnimBoneCompressionSettings.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"

#include "ACLImpl.h"
#include "EditorDatabaseMonitor.h"
#include "UObject/ObjectSaveContext.h"

THIRD_PARTY_INCLUDES_START
#include <acl/compression/compress.h>
#include <acl/compression/track_error.h>
#include <acl/decompression/decompress.h>
#include <acl/decompression/database/null_database_streamer.h>
THIRD_PARTY_INCLUDES_END
#endif	// WITH_EDITORONLY_DATA

#include "ACLDecompressionImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBoneCompressionCodec_ACLDatabase)

void FACLDatabaseCompressedAnimData::SerializeCompressedData(UObject* DataOwner, FArchive& Ar)
{
	FACLCompressedAnimDataBase::SerializeCompressedData(DataOwner, Ar);

	Ar << SequenceNameHash;

#if WITH_EDITORONLY_DATA
	if (!Ar.IsFilterEditorOnly())
	{
		Ar << CompressedClip;
	}
#endif
}

void FACLDatabaseCompressedAnimData::Bind(const TArrayView<uint8> BulkData)
{
	check(BulkData.Num() == 0);	// Should always be empty

#if WITH_EDITORONLY_DATA
	// We have fresh new compressed data which means either we ran compression or we loaded from the DDC
	// We can't tell which is which so mark the database as being potentially dirty
	EditorDatabaseMonitor::MarkDirty(Codec->DatabaseAsset);
#else
	// In a cooked build, we lookup our anim sequence and database from the database asset
	// We search by the sequence hash which lives in the top 32 bits of each entry
	const int32 SequenceIndex = Algo::BinarySearchBy(Codec->DatabaseAsset->CookedAnimSequenceMappings, SequenceNameHash, [](uint64 InValue) { return uint32(InValue >> 32); });
	if (SequenceIndex != INDEX_NONE)
	{
		const uint32 CompressedClipOffset = uint32(Codec->DatabaseAsset->CookedAnimSequenceMappings[SequenceIndex]);	// Truncate top 32 bits
		uint8* CompressedBytes = Codec->DatabaseAsset->CookedCompressedBytes.GetData() + CompressedClipOffset;

		const acl::compressed_tracks* CompressedClipData = acl::make_compressed_tracks(CompressedBytes);
		check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

		const uint32 CompressedSize = CompressedClipData->get_size();

		CompressedByteStream = TArrayView<uint8>(CompressedBytes, CompressedSize);
		DatabaseContext = &Codec->DatabaseAsset->DatabaseContext;
	}
	else
	{
		// This sequence doesn't live in the database, the mapping must be stale
		UE_LOGF(LogAnimationCompression, Warning, "ACL Database mapping is stale. [0x%X] should be contained but isn't.", SequenceNameHash);

		// Since we have no sequence data, decompression will yield a T-pose
	}
#endif
}

int64 FACLDatabaseCompressedAnimData::GetApproxCompressedSize() const
{
#if WITH_EDITORONLY_DATA
	return CompressedClip.Num();
#else
	return CompressedByteStream.Num();
#endif
}

bool FACLDatabaseCompressedAnimData::IsValid() const
{
	const acl::compressed_tracks* CompressedClipData = GetCompressedTracks();
	if (CompressedClipData == nullptr || CompressedClipData->is_valid(false).any())
	{
		return false;
	}

#if !WITH_EDITORONLY_DATA
	if (DatabaseContext == nullptr)
	{
		return false;
	}
#endif

	return true;
}

#if WITH_EDITORONLY_DATA
FACLDatabaseCompressedAnimData::~FACLDatabaseCompressedAnimData()
{
	// Our compressed data is being destroyed which means either we are unloading or we are about to have
	// new compressed data. If the new codec isn't an ACL instance we have no way of knowing so assume the
	// database is dirty and double check.
	EditorDatabaseMonitor::MarkDirty(Codec->DatabaseAsset);
}
#endif

UAnimBoneCompressionCodec_ACLDatabase::UAnimBoneCompressionCodec_ACLDatabase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, DatabaseAsset(nullptr)
{
}

#if WITH_EDITORONLY_DATA
void UAnimBoneCompressionCodec_ACLDatabase::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);

	// We preload the database asset because we need it loaded during Serialize to lookup the proper sequence data
	if (DatabaseAsset != nullptr)
	{
		OutDeps.Add(DatabaseAsset);
	}
}

void UAnimBoneCompressionCodec_ACLDatabase::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	UAnimBoneCompressionSettings* Settings = Cast<UAnimBoneCompressionSettings>(GetOuter());
	if (Settings != nullptr && Settings->Codecs.Num() != 1)
	{
		UE_LOGF(LogAnimationCompression, Error, "ACL database codec must be the only codec in its parent bone compression settings asset. [%ls]", *Settings->GetPathName());
	}
}

void UAnimBoneCompressionCodec_ACLDatabase::PostCompression(const FCompressibleAnimData& CompressibleAnimData, FCompressibleAnimDataResult& OutResult) const
{
	// After we are done compressing our animation sequence, it will contain the necessary metadata needed to build our
	// streaming database. The anim sequence will contain every sample and it will be used as-is in the editor where we
	// show the highest quality by default.
	//
	// However, the anim sequence data that we just compressed will not be used in a cooked build. When we build our
	// database, the sequence data will be modified since we'll remove key frames from it. Its hash will change.
	// The new compressed data will live in the database asset next to the compressed database data. This has the benefit
	// that every compressed clip and the database now live in the same region of virtual memory, reducing the TLB miss
	// rate (when large pages are used on console and mobile since multiple clips fit within a page) and when we do miss
	// the TLB, it will be cheaper since most of the mapping levels are shared.
	//
	// To that end, the data we just compressed will not be serialized in cooked builds, it only lives in the DDC and in memory
	// while in the editor. To be able to find our new sequence data at runtime, we compute the hash from the sequence name.

	FACLDatabaseCompressedAnimData& AnimData = static_cast<FACLDatabaseCompressedAnimData&>(*OutResult.AnimData);

	// Store the sequence full name's hash since we need it in cooked builds to find our data
	AnimData.SequenceNameHash = GetTypeHash(CompressibleAnimData.FullName);

	// Copy the sequence data
	AnimData.CompressedClip = OutResult.CompressedByteStream;

	// When we have a database, the compressed sequence data lives in the database, zero out the compressed byte buffer
	// since we handle the data manually
	OutResult.CompressedByteStream.Empty(0);
}

void UAnimBoneCompressionCodec_ACLDatabase::GetCompressionSettings(const class ITargetPlatform* TargetPlatform, acl::compression_settings& OutSettings) const
{
	OutSettings = acl::get_default_compression_settings();

	OutSettings.level = GetCompressionLevel(CompressionLevel);
	OutSettings.enable_database_support = true;

	// Disable keyframe stripping, even the trivial one as it currently isn't supported
	OutSettings.keyframe_stripping.strip_trivial = false;
}

void UAnimBoneCompressionCodec_ACLDatabase::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	Super::PopulateDDCKey(KeyArgs, Ar);

	acl::compression_settings Settings;
	GetCompressionSettings(KeyArgs.TargetPlatform, Settings);

	uint32 ForceRebuildVersion = 4;
	uint32 SettingsHash = Settings.get_hash();

	Ar	<< ForceRebuildVersion << SettingsHash;

	for (USkeletalMesh* SkelMesh : OptimizationTargets)
	{
		FSkeletalMeshModel* MeshModel = SkelMesh != nullptr ? SkelMesh->GetImportedModel() : nullptr;
		if (MeshModel != nullptr)
		{
			Ar << MeshModel->SkeletalMeshModelGUID;
		}
	}
}
#endif // WITH_EDITORONLY_DATA

TUniquePtr<ICompressedAnimData> UAnimBoneCompressionCodec_ACLDatabase::AllocateAnimData() const
{
	TUniquePtr<FACLDatabaseCompressedAnimData> AnimData = MakeUnique<FACLDatabaseCompressedAnimData>();

	AnimData->Codec = const_cast<UAnimBoneCompressionCodec_ACLDatabase*>(this);

	return AnimData;
}

void UAnimBoneCompressionCodec_ACLDatabase::ByteSwapIn(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryReader& MemoryStream) const
{
#if !PLATFORM_LITTLE_ENDIAN
#error "ACL does not currently support big-endian platforms"
#endif

	// ByteSwapIn(..) is called on load

	// TODO: ACL does not support byte swapping

	// Because we manage the memory manually, the compressed data should always be empty
	check(CompressedData.Num() == 0);
}

void UAnimBoneCompressionCodec_ACLDatabase::ByteSwapOut(ICompressedAnimData& AnimData, TArrayView<uint8> CompressedData, FMemoryWriter& MemoryStream) const
{
#if !PLATFORM_LITTLE_ENDIAN
#error "ACL does not currently support big-endian platforms"
#endif

	// ByteSwapOut(..) is called on save, during cooking, or when counting memory

	// TODO: ACL does not support byte swapping

#if WITH_EDITORONLY_DATA
	// In the editor, if we are saving or cooking, the output should be empty since we manage the memory manually.
	// The real editor data lives in FACLDatabaseCompressedAnimData::CompressedClip
	//
	// Sadly, we have no way of knowing if we are counting memory from here and as such we'll contribute no size.
	// For a true memory report, it is best to run it with cooked data anyway.
	check(CompressedData.Num() == 0);
#else
	// With cooked data, we are never saving unless it is to count memory.
	// Since the actual sequence data lives in the database, its size will be tracked there.
	// We'll do nothing here to avoid counting twice.
#endif
}

void UAnimBoneCompressionCodec_ACLDatabase::DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs, TArrayView<FTransform>& OutAtoms) const
{
	const FACLDatabaseCompressedAnimData& AnimData = static_cast<const FACLDatabaseCompressedAnimData&>(DecompContext.CompressedAnimData);

	acl::decompression_context<UEDefaultDBDecompressionSettings> ACLContext;

#if WITH_EDITORONLY_DATA
	acl::database_context<UEDefaultDatabaseSettings>* DatabaseContext = DatabaseAsset != nullptr ? &DatabaseAsset->DatabaseContext : nullptr;
	if (DatabaseContext != nullptr && DatabaseContext->is_initialized())
	{
		// We are previewing, use the database and the anim sequence data contained within it

		// Lookup our anim sequence from the database asset
		// We search by the sequence hash which lives in the top 32 bits of each entry
		const int32 SequenceIndex = Algo::BinarySearchBy(DatabaseAsset->PreviewAnimSequenceMappings, AnimData.SequenceNameHash, [](uint64 InValue) { return uint32(InValue >> 32); });
		if (SequenceIndex != INDEX_NONE)
		{
			const uint32 CompressedClipOffset = uint32(DatabaseAsset->PreviewAnimSequenceMappings[SequenceIndex]);	// Truncate top 32 bits
			const uint8* CompressedBytes = DatabaseAsset->PreviewCompressedBytes.GetData() + CompressedClipOffset;

			const acl::compressed_tracks* CompressedClipData = acl::make_compressed_tracks(CompressedBytes);
			check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

			ACLContext.initialize(*CompressedClipData, *DatabaseContext);
		}
	}

	if (!ACLContext.is_initialized())
	{
		// No preview or we live updated things and the monitor hasn't caught up yet
		// Use the full quality that lives in the anim sequence
		const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
		check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

		ACLContext.initialize(*CompressedClipData);
	}
#else
	if (AnimData.CompressedByteStream.Num() == 0)
	{
		return;	// Our mapping must have been stale
	}

	const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
	check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

	if (AnimData.DatabaseContext == nullptr || !ACLContext.initialize(*CompressedClipData, *AnimData.DatabaseContext))
	{
		UE_LOGF(LogAnimationCompression, Warning, "ACL failed initialize decompression context, database won't be used");

		ACLContext.initialize(*CompressedClipData);
	}
#endif

	::DecompressPose(DecompContext, ACLContext, RotationPairs, TranslationPairs, ScalePairs, OutAtoms);
}

void UAnimBoneCompressionCodec_ACLDatabase::DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const
{
	const FACLDatabaseCompressedAnimData& AnimData = static_cast<const FACLDatabaseCompressedAnimData&>(DecompContext.CompressedAnimData);

	acl::decompression_context<UEDefaultDBDecompressionSettings> ACLContext;

#if WITH_EDITORONLY_DATA
	acl::database_context<UEDefaultDatabaseSettings>* DatabaseContext = DatabaseAsset != nullptr ? &DatabaseAsset->DatabaseContext : nullptr;
	if (DatabaseContext != nullptr && DatabaseContext->is_initialized())
	{
		// We are previewing, use the database and the anim sequence data contained within it

		// Lookup our anim sequence from the database asset
		// We search by the sequence hash which lives in the top 32 bits of each entry
		const int32 SequenceIndex = Algo::BinarySearchBy(DatabaseAsset->PreviewAnimSequenceMappings, AnimData.SequenceNameHash, [](uint64 InValue) { return uint32(InValue >> 32); });
		if (SequenceIndex != INDEX_NONE)
		{
			const uint32 CompressedClipOffset = uint32(DatabaseAsset->PreviewAnimSequenceMappings[SequenceIndex]);	// Truncate top 32 bits
			const uint8* CompressedBytes = DatabaseAsset->PreviewCompressedBytes.GetData() + CompressedClipOffset;

			const acl::compressed_tracks* CompressedClipData = acl::make_compressed_tracks(CompressedBytes);
			check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

			ACLContext.initialize(*CompressedClipData, *DatabaseContext);
		}
	}

	if (!ACLContext.is_initialized())
	{
		// No preview or we live updated things and the monitor hasn't caught up yet
		// Use the full quality that lives in the anim sequence
		const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
		check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

		ACLContext.initialize(*CompressedClipData);
	}
#else
	if (AnimData.CompressedByteStream.Num() == 0)
	{
		return;	// Our mapping must have been stale
	}

	const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
	check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

	if (AnimData.DatabaseContext == nullptr || !ACLContext.initialize(*CompressedClipData, *AnimData.DatabaseContext))
	{
		UE_LOGF(LogAnimationCompression, Warning, "ACL failed initialize decompression context, database won't be used");

		ACLContext.initialize(*CompressedClipData);
	}
#endif

	::DecompressBone(DecompContext, ACLContext, TrackIndex, OutAtom);
}
