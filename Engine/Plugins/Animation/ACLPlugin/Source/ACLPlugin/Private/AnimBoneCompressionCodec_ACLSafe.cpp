// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "AnimBoneCompressionCodec_ACLSafe.h"

#include "ACLDecompressionImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimBoneCompressionCodec_ACLSafe)

#if WITH_EDITORONLY_DATA
THIRD_PARTY_INCLUDES_START
#include <acl/compression/compression_settings.h>
THIRD_PARTY_INCLUDES_END
#endif

UAnimBoneCompressionCodec_ACLSafe::UAnimBoneCompressionCodec_ACLSafe(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITORONLY_DATA
void UAnimBoneCompressionCodec_ACLSafe::GetCompressionSettings(const class ITargetPlatform* TargetPlatform, acl::compression_settings& OutSettings) const
{
	OutSettings = acl::get_default_compression_settings();

	// Fallback to full precision rotations
	OutSettings.rotation_format = acl::rotation_format8::quatf_full;
}

// @third party code - Epic Games Begin
void UAnimBoneCompressionCodec_ACLSafe::PopulateDDCKey(const UE::Anim::Compression::FAnimDDCKeyArgs& KeyArgs, FArchive& Ar)
{
	Super::PopulateDDCKey(KeyArgs, Ar);

	acl::compression_settings Settings;
	GetCompressionSettings(KeyArgs.TargetPlatform, Settings);

	uint32 ForceRebuildVersion = 1;
	uint32 SettingsHash = Settings.get_hash();

	Ar << DefaultVirtualVertexDistance << SafeVirtualVertexDistance
		<< ForceRebuildVersion << SettingsHash;
}
#endif // WITH_EDITORONLY_DATA

void UAnimBoneCompressionCodec_ACLSafe::DecompressPose(FAnimSequenceDecompressionContext& DecompContext, const BoneTrackArray& RotationPairs, const BoneTrackArray& TranslationPairs, const BoneTrackArray& ScalePairs, TArrayView<FTransform>& OutAtoms) const
{
	const FACLCompressedAnimData& AnimData = static_cast<const FACLCompressedAnimData&>(DecompContext.CompressedAnimData);
	const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
	check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

	acl::decompression_context<UESafeDecompressionSettings> ACLContext;
	ACLContext.initialize(*CompressedClipData);

	::DecompressPose(DecompContext, ACLContext, RotationPairs, TranslationPairs, ScalePairs, OutAtoms);
}

void UAnimBoneCompressionCodec_ACLSafe::DecompressBone(FAnimSequenceDecompressionContext& DecompContext, int32 TrackIndex, FTransform& OutAtom) const
{
	const FACLCompressedAnimData& AnimData = static_cast<const FACLCompressedAnimData&>(DecompContext.CompressedAnimData);
	const acl::compressed_tracks* CompressedClipData = AnimData.GetCompressedTracks();
	check(CompressedClipData != nullptr && CompressedClipData->is_valid(false).empty());

	acl::decompression_context<UESafeDecompressionSettings> ACLContext;
	ACLContext.initialize(*CompressedClipData);

	::DecompressBone(DecompContext, ACLContext, TrackIndex, OutAtom);
}
