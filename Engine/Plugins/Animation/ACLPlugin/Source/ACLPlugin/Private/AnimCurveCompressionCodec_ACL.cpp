// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "AnimCurveCompressionCodec_ACL.h"

#include "ACLImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimCurveCompressionCodec_ACL)

#include "AnimationCompression.h"
#include "Animation/AnimCurveUtils.h"
#include "AnimationCompression.h"

#if WITH_EDITORONLY_DATA
#include "Animation/AnimSequence.h"
#include "Animation/MorphTarget.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshModel.h"

THIRD_PARTY_INCLUDES_START
#include <acl/compression/compress.h>
#include <acl/compression/track.h>
#include <acl/compression/track_array.h>
#include <acl/compression/track_error.h>
#include <acl/core/compressed_tracks_version.h>
THIRD_PARTY_INCLUDES_END
#endif

THIRD_PARTY_INCLUDES_START
#include <acl/decompression/decompress.h>
THIRD_PARTY_INCLUDES_END

UAnimCurveCompressionCodec_ACL::UAnimCurveCompressionCodec_ACL(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	CurvePrecision = 0.001f;
	MorphTargetPositionPrecision = 0.01f;		// 0.01cm, conservative enough for cinematographic quality
#endif
}

#if WITH_EDITORONLY_DATA
void UAnimCurveCompressionCodec_ACL::PopulateDDCKey(FArchive& Ar)
{
	Super::PopulateDDCKey(Ar);

	Ar << CurvePrecision;
	Ar << MorphTargetPositionPrecision;

	if (MorphTargetSource != nullptr)
	{
		FSkeletalMeshModel* MeshModel = MorphTargetSource->GetImportedModel();
		if (MeshModel != nullptr)
		{
			Ar << MeshModel->SkeletalMeshModelGUID;
		}
	}

	uint32 ForceRebuildVersion = 3;
	Ar << ForceRebuildVersion;

	uint16 LatestACLVersion = static_cast<uint16>(acl::compressed_tracks_version16::latest);
	Ar << LatestACLVersion;

	acl::compression_settings Settings;
	uint32 SettingsHash = Settings.get_hash();
	Ar << SettingsHash;
}

// For each curve, returns its largest position delta if the curve is for a morph target, 0.0 otherwise
static TArray<float> GetMorphTargetMaxPositionDeltas(const FCompressibleAnimData& AnimSeq, const USkeletalMesh* MorphTargetSource)
{
	const int32 NumCurves = AnimSeq.RawFloatCurves.Num();

	TArray<float> MorphTargetMaxPositionDeltas;
	MorphTargetMaxPositionDeltas.AddZeroed(NumCurves);

	if (MorphTargetSource == nullptr)
	{
		return MorphTargetMaxPositionDeltas;
	}

	for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		float MaxDeltaPosition = 0.0f;

		const FFloatCurve& Curve = AnimSeq.RawFloatCurves[CurveIndex];
		UMorphTarget* Target = MorphTargetSource->FindMorphTarget(Curve.GetName());
		if (Target != nullptr)
		{
			// This curve drives a morph target, find the largest displacement it can have
			constexpr int32 LODIndex = 0;
			for (const FMorphTargetDelta& Delta: Target->GetMorphTargetDeltas(LODIndex))
			{
				MaxDeltaPosition = FMath::Max(MaxDeltaPosition, Delta.PositionDelta.Size());
			}
		}

		MorphTargetMaxPositionDeltas[CurveIndex] = MaxDeltaPosition;
	}

	return MorphTargetMaxPositionDeltas;
}

static void ResetTracksToIdentity(acl::track_array_float1f& ACLTracks)
{
	// This resets the input ACL track samples to the identity value but retains all other values
	const uint32 NumSamples = 1;
	const float SampleRate = 30.0f;

	const float IdentityValue = 0.0f;

	const acl::track_desc_scalarf DefaultDesc;

	for (acl::track_float1f& ACLTrack : ACLTracks)
	{
		// Reset everything to the identity value and default values
		// Retain the output index to ensure proper output size
		acl::track_desc_scalarf Desc = ACLTrack.get_description();	// Copy
		Desc.precision = DefaultDesc.precision;

		// Reset track to a single sample
		ACLTrack = acl::track_float1f::make_reserve(Desc, ACLAllocatorImpl, NumSamples, SampleRate);
		ACLTrack[0] = IdentityValue;
	}
}

bool UAnimCurveCompressionCodec_ACL::Compress(const FCompressibleAnimData& AnimSeq, FAnimCurveCompressionResult& OutResult)
{
	const TArray<float> MorphTargetMaxPositionDeltas = GetMorphTargetMaxPositionDeltas(AnimSeq, MorphTargetSource);

	const int32 NumCurves = AnimSeq.RawFloatCurves.Num();
	if (NumCurves == 0)
	{
		// Nothing to compress
		OutResult.CompressedBytes.Empty(0);
		OutResult.Codec = this;
		return true;
	}

	const int32 NumSamples = AnimSeq.NumberOfKeys;
	const float SequenceLength = AnimSeq.SequenceLength;

	const bool bIsStaticPose = NumSamples <= 1 || SequenceLength < 0.0001f;
	const float SampleRate = bIsStaticPose ? 30.0f : (float(NumSamples - 1) / SequenceLength);
	const float InvSampleRate = 1.0f / SampleRate;

	acl::track_array_float1f Tracks(ACLAllocatorImpl, NumCurves);

	for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		const FFloatCurve& Curve = AnimSeq.RawFloatCurves[CurveIndex];
		const float MaxPositionDelta = MorphTargetMaxPositionDeltas[CurveIndex];

		// If our curve drives a morph target, we use a different precision value with world space units.
		// This is much easier to tune and control: 0.1mm precision is clear.
		// In order to this this, we must convert that precision value into a value that makes sense for the curve
		// since the animated blend weight doesn't have any units: it's a scaling factor.
		// The morph target math is like this for every vertex: result vtx = ref vtx + (target vtx - ref vtx) * blend weight
		// (target vtx - ref vtx) is our deformation difference (or delta) and we scale it between 0.0 and 1.0 with our blend weight.
		// At 0.0, the resulting vertex is 100% the reference vertex.
		// At 1.0, the resulting vertex is 100% the target vertex.
		// This can thus be re-written as follow: result vtx = ref vtx + vtx delta * blend weight
		// From this, it follows that any error we introduce into the blend weight will impact the delta linearly.
		// If our delta measures 1 meter, an error of 10% translates into 0.1 meter.
		// If our delta measures 1 cm, an error of 10% translates into 0.1 cm.
		// Thus, for a given error quantity, a larger delta means a larger resulting difference from the original value.
		// If the delta is zero, any error is irrelevant as it will have no measurable impact.
		// By dividing the precision value we want with the delta length, we can control how much precision our blend weight needs.
		// If we want 0.01 cm precision and our largest vertex displacement is 3 cm, the blend weight precision needs to be:
		// 0.01 cm / 3.00 cm = 0.0033 (with the units canceling out just like we need)
		// Another way to think about it is that every 0.0033 increment of the blend weight results in an increment of 0.01 cm
		// when our displacement delta is 3 cm.
		// 0.01 cm / 50.00 cm = 0.0002 (if our delta increases, we need to retain more blend weight precision)
		// 0.01 cm / 1.00 cm = 0.01
		// Each blend weight curve will drive a different target position for many vertices and this way, we can specify
		// a single value for the world space precision we want to achieve for every vertex and every blend weight curve
		// will end up with the precision value it needs.
		//
		// If our curve doesn't drive a morph target, we use the supplied CurvePrecision instead.

		const float Precision = MaxPositionDelta > 0.0f ? (MorphTargetPositionPrecision / MaxPositionDelta) : CurvePrecision;

		acl::track_desc_scalarf Desc;
		Desc.output_index = CurveIndex;
		Desc.precision = Precision;

		acl::track_float1f Track = acl::track_float1f::make_reserve(Desc, ACLAllocatorImpl, NumSamples, SampleRate);
		for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
		{
			const float SampleTime = FMath::Clamp(SampleIndex * InvSampleRate, 0.0f, SequenceLength);
			const float SampleValue = Curve.FloatCurve.Eval(SampleTime);

			Track[SampleIndex] = SampleValue;
		}

		Tracks[CurveIndex] = MoveTemp(Track);
	}

	acl::compression_settings Settings;

	acl::compressed_tracks* CompressedTracks = nullptr;
	acl::output_stats Stats;
	acl::error_result CompressionResult = acl::compress_track_list(ACLAllocatorImpl, Tracks, Settings, CompressedTracks, Stats);

	bool bEnableErrorReporting = true;
	bool bCompressionFailed = false;

	if (CompressionResult.any())
	{
		// If compression failed, one of two things happened:
		//    * Invalid settings were used, this would be a code/logic error that results in an improper usage of ACL
		//    * Invalid data was provided, this would be a validation error that should ideally be caught earlier (e.g import, save)
		// 
		// Either way, if we get here, we cannot recover and we cannot fail as the engine assumes that compression always succeeds.
		// We must handle failure gracefully. To that end, we compress an empty stub to ensure that something is present to
		// decompress. Because the stub is empty, we'll simply output the default values. We still log this as an error to signal that
		// this is a problem that needs to be fixed. This will allow the editor to continue working with the default values we'll output
		// but cooking will fail preventing us from running with invalid state.

		UE_LOGF(LogAnimationCompression, Error, "ACL failed to compress curves: %ls [%ls]", ANSI_TO_TCHAR(CompressionResult.c_str()), *AnimSeq.FullName);

		// We reset the tracks to the identity, getting rid of any potentially invalid data.
		ResetTracksToIdentity(Tracks);

		CompressionResult = acl::compress_track_list(ACLAllocatorImpl, Tracks, Settings, CompressedTracks, Stats);

		// The stub compression should never fail
		check(CompressionResult.empty() && CompressedTracks != nullptr);

		// Because we compress an empty stub, disable error reporting below
		bEnableErrorReporting = false;
		bCompressionFailed = true;
	}

	checkSlow(CompressedTracks->is_valid(true).empty());

	const uint32 CompressedDataSize = CompressedTracks->get_size();

	// When compression fails, we add an extra few bytes of padding at the end
	// This allows us to detect that the size is different so that we can output an error when validating
	const uint32 ErrorPaddingValue = 0xFAFACDCD;
	const uint32 ErrorPaddingSize = bCompressionFailed ? sizeof(ErrorPaddingValue) : 0;

	OutResult.CompressedBytes.Empty(CompressedDataSize + ErrorPaddingSize);
	OutResult.CompressedBytes.AddUninitialized(CompressedDataSize + ErrorPaddingSize);
	FMemory::Memcpy(OutResult.CompressedBytes.GetData(), CompressedTracks, CompressedDataSize);

	if (bCompressionFailed)
	{
		// Ensure our padding is deterministic (might not be aligned)
		FMemory::Memcpy(OutResult.CompressedBytes.GetData() + CompressedDataSize, &ErrorPaddingValue, sizeof(ErrorPaddingValue));
	}

	OutResult.Codec = this;

#if !NO_LOGGING
	if (bEnableErrorReporting && LogAnimationCompression.GetVerbosity() >= ELogVerbosity::Verbose)
	{
		acl::decompression_context<acl::debug_scalar_decompression_settings> Context;
		Context.initialize(*CompressedTracks);
		const acl::track_error Error = acl::calculate_compression_error(ACLAllocatorImpl, Tracks, Context);

		UE_LOGF(LogAnimationCompression, Verbose, "ACL Curves compressed size: %u bytes [%ls]", CompressedDataSize, *AnimSeq.FullName);
		UE_LOGF(LogAnimationCompression, Verbose, "ACL Curves error: %.4f (curve %u @ %.3f) [%ls]", Error.error, Error.index, Error.sample_time, *AnimSeq.FullName);
	}
#endif

	ACLAllocatorImpl.deallocate(CompressedTracks, CompressedDataSize);
	return true;
}

int64 UAnimCurveCompressionCodec_ACL::EstimateCompressionMemoryUsage(const UAnimSequence& AnimSequence) const
{
	const int64 NumCurves = AnimSequence.GetDataModelInterface()->GetFloatCurves().Num();
	const int64 NumSamples = AnimSequence.GetNumberOfSampledKeys();

	const int64 RawDataSize = NumCurves * NumSamples;

	int64 EstimatedMemoryUsage = 0;
	EstimatedMemoryUsage += RawDataSize;	// We copy the raw data into the ACL format
	EstimatedMemoryUsage *= 2;				// Internally, ACL copies the raw data into a different format than the input because the input is not modified
	EstimatedMemoryUsage += RawDataSize;	// ACL keeps a mutable copy of the lossy data that it modifies during compression
	EstimatedMemoryUsage += RawDataSize;	// ACL will allocate the output buffer, assume that it's as large as the raw data
	EstimatedMemoryUsage += 100 * 1024;		// Reserve 100 KB for internal bookkeeping and other required metadata

	return EstimatedMemoryUsage;
}
#endif // WITH_EDITORONLY_DATA

bool UAnimCurveCompressionCodec_ACL::ValidateCompressedData(UObject* DataOwner, const FCompressedAnimSequence& AnimSeq) const
{
	if (AnimSeq.IndexedCurveNames.Num() == 0)
	{
		return true;
	}

	const acl::compressed_tracks* CompressedTracks = acl::make_compressed_tracks(AnimSeq.CompressedCurveByteStream.GetData());
	if (CompressedTracks == nullptr || CompressedTracks->is_valid(false).any())
	{
		UE_LOGF(LogAnimationCompression, Error,
			"ACL compressed curve data is missing or corrupted for an anim sequence: %ls",
			DataOwner != nullptr ? *DataOwner->GetPathName() : TEXT("[Unknown Sequence]"));
		return false;
	}

	// Check if we have padding and if it has our magic value to signal failure (might not be aligned)
	const uint32 CompressedSize = CompressedTracks->get_size();
	const uint32 CompressedCurveByteStreamSize = AnimSeq.CompressedCurveByteStream.Num();
	const uint32 ErrorPaddingValue = (CompressedSize + sizeof(uint32)) <= CompressedCurveByteStreamSize ?
		*reinterpret_cast<const uint32*>(&AnimSeq.CompressedCurveByteStream[CompressedSize]) : 0;
	if (ErrorPaddingValue == 0xFAFACDCD)
	{
		UE_LOGF(LogAnimationCompression, Error,
			"ACL failed to compress curves for an anim sequence and will output the default curve values at runtime: %ls",
			DataOwner != nullptr ? *DataOwner->GetPathName() : TEXT("[Unknown Sequence]"));
		return false;
	}

	// All good!
	return true;
}

struct UECurveDecompressionSettings final : public acl::decompression_settings
{
	static constexpr bool is_track_type_supported(acl::track_type8 type) { return type == acl::track_type8::float1f; }

	// Only support our latest version
	static constexpr acl::compressed_tracks_version16 version_supported() { return acl::compressed_tracks_version16::latest; }

#if UE_BUILD_SHIPPING
	// Shipping builds do not need safety checks, by then the game has been tested enough
	// Only data corruption could cause a safety check to fail
	// We keep this disabled regardless because it is generally better to output a T-pose than to have a
	// potential crash. Corruption can happen and it would be unfortunate if a demo or playtest failed
	// as a result of a crash that we can otherwise recover from.
	//static constexpr bool skip_initialize_safety_checks() { return true; }
#endif
};

struct UECurveWriter final : public acl::track_writer
{
	TArray<float, FAnimStackAllocator>& Buffer;

	explicit UECurveWriter(TArray<float, FAnimStackAllocator>& Buffer_)
		: Buffer(Buffer_)
	{
	}

	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_float1(uint32_t TrackIndex, rtm::scalarf_arg0 Value)
	{
		Buffer[TrackIndex] = rtm::scalar_cast(Value);
	}
};

void UAnimCurveCompressionCodec_ACL::DecompressCurves(const FCompressedAnimSequence& AnimSeq, FBlendedCurve& Curves, float CurrentTime) const
{
	const TArray<FAnimCompressedCurveIndexedName>& IndexedCurveNames = AnimSeq.IndexedCurveNames;
	const int32 NumCurves = IndexedCurveNames.Num();

	if (NumCurves == 0)
	{
		return;
	}

	const acl::compressed_tracks* CompressedTracks = acl::make_compressed_tracks(AnimSeq.CompressedCurveByteStream.GetData());
	check(CompressedTracks != nullptr && CompressedTracks->is_valid(false).empty());

	acl::decompression_context<UECurveDecompressionSettings> Context;
	Context.initialize(*CompressedTracks);
	Context.seek(CurrentTime, acl::sample_rounding_policy::none);

	TArray<float, FAnimStackAllocator> DecompressionBuffer;
	DecompressionBuffer.SetNumUninitialized(NumCurves);

	UECurveWriter TrackWriter(DecompressionBuffer);
	Context.decompress_tracks(TrackWriter);

	auto GetNameFromIndex = [&IndexedCurveNames](int32 InCurveIndex)
	{
		return IndexedCurveNames[IndexedCurveNames[InCurveIndex].CurveIndex].CurveName;
	};

	auto GetValueFromIndex = [&DecompressionBuffer, &IndexedCurveNames](int32 InCurveIndex)
	{
		return DecompressionBuffer[IndexedCurveNames[InCurveIndex].CurveIndex];
	};

	UE::Anim::FCurveUtils::BuildSorted(Curves, NumCurves, GetNameFromIndex, GetValueFromIndex, Curves.GetFilter());
}

struct UEScalarCurveWriter final : public acl::track_writer
{
	float SampleValue;

	UEScalarCurveWriter()
		: SampleValue(0.0f)
	{
	}

	FORCEINLINE_DEBUGGABLE void RTM_SIMD_CALL write_float1(uint32_t /*TrackIndex*/, rtm::scalarf_arg0 Value)
	{
		SampleValue = rtm::scalar_cast(Value);
	}
};

float UAnimCurveCompressionCodec_ACL::DecompressCurve(const FCompressedAnimSequence& AnimSeq, FName CurveName, float CurrentTime) const
{
	const TArray<FAnimCompressedCurveIndexedName>& IndexedCurveNames = AnimSeq.IndexedCurveNames;
	const int32 NumCurves = IndexedCurveNames.Num();

	if (NumCurves == 0)
	{
		return 0.0f;
	}

	const acl::compressed_tracks* CompressedTracks = acl::make_compressed_tracks(AnimSeq.CompressedCurveByteStream.GetData());
	check(CompressedTracks != nullptr && CompressedTracks->is_valid(false).empty());

	acl::decompression_context<UECurveDecompressionSettings> Context;
	Context.initialize(*CompressedTracks);
	Context.seek(CurrentTime, acl::sample_rounding_policy::none);

	int32 TrackIndex = -1;
	for (int32 CurveIndex = 0; CurveIndex < NumCurves; ++CurveIndex)
	{
		if (IndexedCurveNames[CurveIndex].CurveName == CurveName)
		{
			TrackIndex = CurveIndex;
			break;
		}
	}

	if (TrackIndex < 0)
	{
		return 0.0f;	// Track not found
	}

	UEScalarCurveWriter TrackWriter;
	Context.decompress_track(TrackIndex, TrackWriter);

	return TrackWriter.SampleValue;
}
