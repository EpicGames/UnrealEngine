// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "AnimationCompressionLibraryDatabase.h"
#include "AnimBoneCompressionCodec_ACLDatabase.h"
#include "Engine/Engine.h"
#include "UEDatabaseStreamer.h"

#include "LatentActions.h"
#include "Containers/Ticker.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Animation/IAnimationSequenceCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationCompressionLibraryDatabase)

#if WITH_EDITORONLY_DATA
#include "Animation/AnimBoneCompressionSettings.h"
#include "PlatformInfo.h"
#include "Interfaces/ITargetPlatform.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectIterator.h"

#include "ACLImpl.h"
#include "UEDatabasePreviewStreamer.h"

THIRD_PARTY_INCLUDES_START
#include <acl/compression/compress.h>
THIRD_PARTY_INCLUDES_END
#endif	// WITH_EDITORONLY_DATA

const TCHAR* VisualFidelityToString(ACLVisualFidelity Fidelity)
{
	switch (Fidelity)
	{
	case ACLVisualFidelity::Highest:
		return TEXT("Highest");
	case ACLVisualFidelity::Medium:
		return TEXT("Medium");
	case ACLVisualFidelity::Lowest:
		return TEXT("Lowest");
	default:
		return TEXT("Unknown visual fidelity");
	}
}

UAnimationCompressionLibraryDatabase::UAnimationCompressionLibraryDatabase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CurrentVisualFidelity(ACLVisualFidelity::Lowest)
	, NextFidelityChangeRequestID(0)
#if WITH_EDITORONLY_DATA
	, HighestImportanceProportion(0.5f)	// The rest remains in the anim sequences
	, MediumImportanceProportion(0.0f)	// No medium quality tier by default
	, LowestImportanceProportion(0.5f)	// By default we move 50% of the key frames to the database
	, StripLowestImportanceTier(false)	// By default we don't strip the lowest tier
#endif
	, MaxStreamRequestSizeKB(1024)		// By default we stream 1 MB (1 chunk) at a time
	, DefaultVisualFidelity(ACLVisualFidelity::Lowest)
#if WITH_EDITORONLY_DATA
	// By default, in the editor we preview the full quality.
	// Our database context won't be used until we need to build the database for preview if we change this value.
	, PreviewVisualFidelity(ACLVisualFidelity::Highest)
	, NumAnimSequences(0)
	, AnimSequencesOldSizeKB(0)
	, AnimSequencesNewSizeKB(0)
	, DatabaseSizeKB(0)
	, DatabaseMetadataSizeKB(0)
	, MediumImportanceSizeKB(0)
	, LowImportanceSizeSizeKB(0)
#endif
{
}

#if WITH_EDITORONLY_DATA
void UAnimationCompressionLibraryDatabase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property == nullptr)
	{
		return;	// Safety check
	}

	const FName ChangedPropertyName = PropertyChangedEvent.Property->GetFName();

	bool bBuildDatabaseForPreview = false;
	bool bUpdateStreaming = false;

	if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UAnimationCompressionLibraryDatabase, LowestImportanceProportion))
	{
		// Clamp our medium importance propertion
		MediumImportanceProportion = FMath::Clamp(MediumImportanceProportion, 0.0f, FMath::Clamp(1.0f - LowestImportanceProportion, 0.0f, 1.0f));

		// Update our high importance proportion to reflect what remains
		HighestImportanceProportion = FMath::Clamp(1.0f - LowestImportanceProportion - MediumImportanceProportion, 0.0f, 1.0f);

		// Only update our preview state if we are setting the final value to avoid slowing down the UI when using the slider
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
		{
			bBuildDatabaseForPreview = PreviewDatabaseStreamer != nullptr;	// If we already had a preview database, we need to rebuild it
		}
	}
	else if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UAnimationCompressionLibraryDatabase, MediumImportanceProportion))
	{
		// Clamp our medium importance propertion
		LowestImportanceProportion = FMath::Clamp(LowestImportanceProportion, 0.0f, FMath::Clamp(1.0f - MediumImportanceProportion, 0.0f, 1.0f));

		// Update our high importance proportion to reflect what remains
		HighestImportanceProportion = FMath::Clamp(1.0f - LowestImportanceProportion - MediumImportanceProportion, 0.0f, 1.0f);

		// Only update our preview state if we are setting the final value to avoid slowing down the UI when using the slider
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
		{
			bBuildDatabaseForPreview = PreviewDatabaseStreamer != nullptr;	// If we already had a preview database, we need to rebuild it
		}
	}
	else if (ChangedPropertyName == GET_MEMBER_NAME_CHECKED(UAnimationCompressionLibraryDatabase, PreviewVisualFidelity))
	{
		// Our preview state changed, check if we need to generate our database
		bBuildDatabaseForPreview = PreviewDatabaseStreamer == nullptr;	// We didn't have a preview database, create one now
		bUpdateStreaming = true;
	}

	if (bBuildDatabaseForPreview || bUpdateStreaming)
	{
		UpdatePreviewState(bBuildDatabaseForPreview);
	}
}

void UAnimationCompressionLibraryDatabase::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	// Clear any stale cooked data we might have
	CookedCompressedBytes.Empty(0);
	CookedAnimSequenceMappings.Empty(0);
	CookedBulkData.RemoveBulkData();

	const ITargetPlatform* TargetPlatform = ObjectSaveContext.GetTargetPlatform();
	if (TargetPlatform != nullptr && TargetPlatform->RequiresCookedData())
	{
		const bool bStripLowestTier = StripLowestImportanceTier.GetValueForPlatform(*TargetPlatform->IniPlatformName());

		TArray<uint8> BulkData;
		BuildDatabase(CookedCompressedBytes, CookedAnimSequenceMappings, BulkData, bStripLowestTier);

		CookedBulkData.Lock(LOCK_READ_WRITE);
		{
			const uint32 BulkDataSize = BulkData.Num();
			void* BulkDataToSave = CookedBulkData.Realloc(BulkDataSize);
			FMemory::Memcpy(BulkDataToSave, BulkData.GetData(), BulkDataSize);
		}
		CookedBulkData.Unlock();
	}
}

void UAnimationCompressionLibraryDatabase::BuildDatabase(TArray<uint8>& OutCompressedBytes, TArray<uint64>& OutAnimSequenceMappings, TArray<uint8>& OutBulkData, bool bStripLowestTier)
{
	// Clear any stale data we might have
	OutCompressedBytes.Empty(0);
	OutAnimSequenceMappings.Empty(0);
	OutBulkData.Empty(0);
	NumAnimSequences = 0;
	AnimSequencesOldSizeKB = 0;
	AnimSequencesNewSizeKB = 0;
	DatabaseSizeKB = 0;
	DatabaseMetadataSizeKB = 0;
	MediumImportanceSizeKB = 0;
	LowImportanceSizeSizeKB = 0;

	// We are cooking or previewing, iterate over every animation sequence that references this database
	// and merge them together into our final database instance. Note that the mapping could
	// be stale and we must double check.

	// Gather the sequences we need to merge, these are already sorted by FName by construction
	TArray<UAnimSequence*> CookedSequences;
	CookedSequences.Empty(AnimSequences.Num());

	// Because we use a hash at runtime in a cooked build to retrieve our clip data, we must ensure that the hash value is unique
	TSet<uint32> SequenceHashes;
	SequenceHashes.Empty(AnimSequences.Num());

	const ITargetPlatform* CurrentPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
	const uint32 LibraryHash = GetTypeHash(this);
		
	// Update our compressed data to match the current settings
	// We might need to revert to non-frame stripped data or the codec might have changed
	// forcing us to recompress
	// In practice we'll load directly from the DDC and it should be fast
	TGuardValue<int32> FrameStrippingGuard(GPerformFrameStripping, 0);
	
	for (UAnimSequence* AnimSeq : AnimSequences)
	{
		AnimSeq->RequestResidency(CurrentPlatform, LibraryHash);
	}

	UE::Anim::IAnimSequenceCompilingManager::FinishCompilation(AnimSequences);

	auto ReleaseResidency = [this, CurrentPlatform, &LibraryHash]()
	{
		for (UAnimSequence* AnimSeq : AnimSequences)
		{
			AnimSeq->ReleaseResidency(CurrentPlatform, LibraryHash);
		}
	};	

	for (UAnimSequence* AnimSeq : AnimSequences)
	{
		UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSeq->GetCompressedData();
		check(CompressedAnimSequence.Get().IsValid(AnimSeq));

		UAnimBoneCompressionCodec_ACLDatabase* DatabaseCodec = Cast<UAnimBoneCompressionCodec_ACLDatabase>(CompressedAnimSequence.Get().BoneCompressionCodec);
		if (DatabaseCodec == nullptr || DatabaseCodec->DatabaseAsset != this)
		{
			UE_LOGF(LogAnimationCompression, Warning, "ACL Database mapping is stale. '%ls' no longer references it. [%ls]", *AnimSeq->GetPathName(), *GetPathName());
			continue;
		}

		const FACLDatabaseCompressedAnimData& AnimData = static_cast<const FACLDatabaseCompressedAnimData&>(*CompressedAnimSequence.Get().CompressedDataStructure);
		if (!AnimData.IsValid())
		{
			UE_LOGF(LogAnimationCompression, Warning, "Cannot include a invalid sequence '%ls' in the ACL database. [%ls]", *AnimSeq->GetPathName(), *GetPathName());
			continue;
		}

		bool bIsAlreadyInSet = false;
		SequenceHashes.Add(AnimData.SequenceNameHash, &bIsAlreadyInSet);
		if (bIsAlreadyInSet)
		{
			// Our anim sequence has a hash that we've seen already which means an anim sequence has been duplicated
			// and lives in a separate asset with identical data.
			// We'll skip it since we'll be able to re-use the same data at runtime.
			continue;
		}

		CookedSequences.Add(AnimSeq);
	}

	if (CookedSequences.Num() == 0)
	{
		return;	// Nothing to cook
	}

	TArray<const acl::compressed_tracks*> ACLCompressedTracks;
	for (const UAnimSequence* AnimSeq : CookedSequences)
	{
		UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSeq->GetCompressedData();
		const FACLDatabaseCompressedAnimData& AnimData = static_cast<const FACLDatabaseCompressedAnimData&>(*CompressedAnimSequence.Get().CompressedDataStructure);
		ACLCompressedTracks.Add(AnimData.GetCompressedTracks());
	}

	const int32 NumSequences = ACLCompressedTracks.Num();

	acl::compression_database_settings Settings;	// Use defaults
	Settings.low_importance_tier_proportion = LowestImportanceProportion;
	Settings.medium_importance_tier_proportion = MediumImportanceProportion;

	TArray<acl::compressed_tracks*> ACLDBCompressedTracks;
	ACLDBCompressedTracks.AddZeroed(NumSequences);

	acl::compressed_database* MergedDB = nullptr;
	acl::error_result MergeResult = acl::build_database(ACLAllocatorImpl, Settings, ACLCompressedTracks.GetData(), NumSequences, ACLDBCompressedTracks.GetData(), MergedDB);

	if (MergeResult.any())
	{
		// Free our duplicate compressed clips
		UE_LOGF(LogAnimationCompression, Error, "ACL failed to merge databases: %ls [%ls]", ANSI_TO_TCHAR(MergeResult.c_str()), *GetPathName());
		return;
	}

#if DO_GUARD_SLOW
	// Sanity check that the database is properly constructed
	{
		checkSlow(MergedDB->is_valid(true).empty());

		acl::database_context<UEDefaultDatabaseSettings> DebugDatabaseContext;
		const bool ContextInitResult = DebugDatabaseContext.initialize(ACLAllocatorImpl, *MergedDB);
		checkf(ContextInitResult, TEXT("ACL failed to initialize the database context"));

		for (const acl::compressed_tracks* CompressedTracks : ACLDBCompressedTracks)
		{
			checkSlow(CompressedTracks->is_valid(true).empty());
			checkSlow(MergedDB->contains(*CompressedTracks));
			checkSlow(DebugDatabaseContext.contains(*CompressedTracks));
		}
}
#endif

	// Split our database to serialize the bulk data separately
	acl::compressed_database* SplitDB = nullptr;
	uint8* SplitDBBulkDataMedium = nullptr;
	uint8* SplitDBBulkDataLow = nullptr;
	const acl::error_result SplitResult = acl::split_database_bulk_data(ACLAllocatorImpl, *MergedDB, SplitDB, SplitDBBulkDataMedium, SplitDBBulkDataLow);

	// Free the merged instance we no longer need
	ACLAllocatorImpl.deallocate(MergedDB, MergedDB->get_size());
	MergedDB = nullptr;

	if (SplitResult.any())
	{
		// Free our clip copies
		for (acl::compressed_tracks* CompressedTracks : ACLDBCompressedTracks)
		{
			ACLAllocatorImpl.deallocate(CompressedTracks, CompressedTracks->get_size());
		}

		UE_LOGF(LogAnimationCompression, Error, "ACL failed to split database: %ls [%ls]", ANSI_TO_TCHAR(SplitResult.c_str()), *GetPathName());

		ReleaseResidency();
		return;
	}

	checkSlow(SplitDB->is_valid(true).empty());

	const uint32 BulkDataSizeMedium = SplitDB->get_bulk_data_size(acl::quality_tier::medium_importance);
	const uint32 BulkDataSizeLow = SplitDB->get_bulk_data_size(acl::quality_tier::lowest_importance);
	uint32 BulkDataEffectiveSizeLow = BulkDataSizeLow;

	// Strip our lowest tier if requested and if it contains data
	if (bStripLowestTier && SplitDB->has_bulk_data(acl::quality_tier::lowest_importance))
	{
		acl::compressed_database* StrippedDB = nullptr;
		const acl::error_result StripResult = acl::strip_database_quality_tier(ACLAllocatorImpl, *SplitDB, acl::quality_tier::lowest_importance, StrippedDB);

		if (StripResult.any())
		{
			// We failed to strip our tier but the split database is still usable, don't fail anything
			UE_LOGF(LogAnimationCompression, Warning, "ACL failed to strip lowest database tier: %ls [%ls]", ANSI_TO_TCHAR(StripResult.c_str()), *GetPathName());
		}
		else
		{
			// Medium tier shouldn't have changed
			check(BulkDataSizeMedium == StrippedDB->get_bulk_data_size(acl::quality_tier::medium_importance));

			// Free the split instance we no longer need
			ACLAllocatorImpl.deallocate(SplitDB, SplitDB->get_size());

			// Use the stripped database instead
			SplitDB = StrippedDB;

			// We no longer have the lowest tier
			BulkDataEffectiveSizeLow = 0;
		}
	}

	const uint32 CompressedDatabaseSize = SplitDB->get_size();

	// Our compressed sequences follow the database in memory, aligned to 16 bytes
	uint32 CompressedSequenceOffset = acl::align_to(CompressedDatabaseSize, 16);

	// Write our our cooked offset mappings
	// We use an array for simplicity. UE4 doesn't support serializing a TMap or TSortedMap and so instead
	// we store an array of sorted FNames hashes and offsets. We'll use binary search to find our
	// index at runtime in O(logN) in the sorted names array, and read the offset we need in the other.
	// TODO: Use perfect hashing to bring it to O(1)

	SIZE_T TotalSizeSeqOld = 0;
	SIZE_T TotalSizeSeqNew = 0;

	OutAnimSequenceMappings.Empty(NumSequences);
	for (int32 MappingIndex = 0; MappingIndex < NumSequences; ++MappingIndex)
	{
		UAnimSequence* AnimSeq = CookedSequences[MappingIndex];

		UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = const_cast<const UAnimSequence*>(AnimSeq)->GetCompressedData();
		const acl::compressed_tracks* CompressedTracks = ACLDBCompressedTracks[MappingIndex];
		FACLDatabaseCompressedAnimData& AnimData = static_cast<FACLDatabaseCompressedAnimData&>(*CompressedAnimSequence.Get().CompressedDataStructure);

		// Align our sequence to 16 bytes
		CompressedSequenceOffset = acl::align_to(CompressedSequenceOffset, 16);

		// Add our mapping
		OutAnimSequenceMappings.Add((uint64(AnimData.SequenceNameHash) << 32) | uint64(CompressedSequenceOffset));

		// Increment our offset but don't align since we don't want to add unnecesary padding at the end of the last sequence
		CompressedSequenceOffset += CompressedTracks->get_size();

		TotalSizeSeqOld += ACLCompressedTracks[MappingIndex]->get_size();
		TotalSizeSeqNew += CompressedTracks->get_size();
	}

	auto BytesToMB = [](SIZE_T NumBytes) { return (double)NumBytes / (1024.0 * 1024.0); };

	UE_LOGF(LogAnimationCompression, Log, "ACL DB [%ls] Sequences (%u) went from %.2f MB -> %.2f MB. DB is %.2f MB",
		*GetPathName(), NumSequences, BytesToMB(TotalSizeSeqOld), BytesToMB(TotalSizeSeqNew), BytesToMB(SplitDB->get_total_size()));
	UE_LOGF(LogAnimationCompression, Log, "    DB metadata is %.2f MB", BytesToMB(SplitDB->get_size()));
	UE_LOGF(LogAnimationCompression, Log, "    DB medium tier is %.2f MB", BytesToMB(BulkDataSizeMedium));
	UE_LOGF(LogAnimationCompression, Log, "    DB lowest tier is %.2f MB%ls", BytesToMB(BulkDataSizeLow), bStripLowestTier ? TEXT(" (stripped)") : TEXT(""));

	auto BytesToCeilKB = [](SIZE_T NumBytes) { return int32((NumBytes + 1023) / 1024); };

	NumAnimSequences = NumSequences;
	AnimSequencesOldSizeKB = BytesToCeilKB(TotalSizeSeqOld);
	AnimSequencesNewSizeKB = BytesToCeilKB(TotalSizeSeqNew);
	DatabaseSizeKB = BytesToCeilKB(SplitDB->get_total_size());
	DatabaseMetadataSizeKB = BytesToCeilKB(SplitDB->get_size());
	MediumImportanceSizeKB = BytesToCeilKB(BulkDataSizeMedium);
	LowImportanceSizeSizeKB = BytesToCeilKB(BulkDataSizeLow);

	// Make sure to sort our array, it'll be sorted by hash first since it lives in the top bits
	OutAnimSequenceMappings.Sort();

	// Our full buffer size is our resulting offset
	const uint32 CompressedBytesSize = CompressedSequenceOffset;

	// Copy our database
	OutCompressedBytes.Empty(CompressedBytesSize);
	OutCompressedBytes.AddZeroed(CompressedBytesSize);
	FMemory::Memcpy(OutCompressedBytes.GetData(), SplitDB, CompressedDatabaseSize);

	// Copy our compressed clips
	CompressedSequenceOffset = acl::align_to(CompressedDatabaseSize, 16);	// Reset

	for (const acl::compressed_tracks* CompressedTracks : ACLDBCompressedTracks)
	{
		// Align our sequence to 16 bytes
		CompressedSequenceOffset = acl::align_to(CompressedSequenceOffset, 16);

		// Copy our data
		FMemory::Memcpy(OutCompressedBytes.GetData() + CompressedSequenceOffset, CompressedTracks, CompressedTracks->get_size());

		// Increment our offset but don't align since we don't want to add unnecesary padding at the end of the last sequence
		CompressedSequenceOffset += CompressedTracks->get_size();
	}

	// Copy our bulk data
	uint32 BulkDataSize = BulkDataSizeMedium;
	if (!bStripLowestTier && BulkDataEffectiveSizeLow != 0)
	{
		BulkDataSize += BulkDataEffectiveSizeLow;
	}

	OutBulkData.Empty(BulkDataSize);
	OutBulkData.AddZeroed(BulkDataSize);
	FMemory::Memcpy(OutBulkData.GetData(), SplitDBBulkDataMedium, BulkDataSizeMedium);

	if (!bStripLowestTier && BulkDataEffectiveSizeLow != 0)
	{
		FMemory::Memcpy(OutBulkData.GetData() + BulkDataSizeMedium, SplitDBBulkDataLow, BulkDataEffectiveSizeLow);
	}

	// Free the split instance we no longer need
	ACLAllocatorImpl.deallocate(SplitDB, CompressedDatabaseSize);
	ACLAllocatorImpl.deallocate(SplitDBBulkDataMedium, BulkDataSizeMedium);
	ACLAllocatorImpl.deallocate(SplitDBBulkDataLow, BulkDataSizeLow);

	// Free our clip copies
	for (acl::compressed_tracks* CompressedTracks : ACLDBCompressedTracks)
	{
		ACLAllocatorImpl.deallocate(CompressedTracks, CompressedTracks->get_size());
	}
	
	ReleaseResidency();
}

void UAnimationCompressionLibraryDatabase::UpdatePreviewState(bool bBuildDatabase)
{
	// Check if we need to build/rebuild our preview database
	if (bBuildDatabase)
	{
		// Create a temporary database now so we can preview our animations at the desired quality
		PreviewDatabaseStreamer.Reset();
		DatabaseContext.reset();

		BuildDatabase(PreviewCompressedBytes, PreviewAnimSequenceMappings, PreviewBulkData);

		if (PreviewCompressedBytes.Num() != 0)
		{
			// Our database was built, initialize what we need to be able to use it
			const acl::compressed_database* CompressedDatabase = acl::make_compressed_database(PreviewCompressedBytes.GetData());
			check(CompressedDatabase != nullptr && CompressedDatabase->is_valid(false).empty());

			PreviewDatabaseStreamer = MakeUnique<UEDatabasePreviewStreamer>(*CompressedDatabase, PreviewBulkData);

			const bool ContextInitResult = DatabaseContext.initialize(ACLAllocatorImpl, *CompressedDatabase, *PreviewDatabaseStreamer, *PreviewDatabaseStreamer);
			checkf(ContextInitResult, TEXT("ACL failed to initialize the database context"));

			// New fidelity is lowest
			CurrentVisualFidelity = ACLVisualFidelity::Lowest;
		}
	}

	// Perform any streaming request as a result of our new preview/database state
	if (DatabaseContext.is_initialized())
	{
		SetVisualFidelity(PreviewVisualFidelity);
	}
}

bool UAnimationCompressionLibraryDatabase::UpdateReferencingAnimSequenceList()
{
	// Grab every anim sequence that references us
	TArray<TObjectPtr<UAnimSequence>> ReferencingAnimSequences;
	for (TObjectIterator<UAnimSequence> It; It; ++It)
	{
		UAnimSequence* AnimSeq = *It;
		if (AnimSeq->GetOutermost() == GetTransientPackage())
		{
			continue;
		}

		UAnimBoneCompressionSettings* Settings = AnimSeq->BoneCompressionSettings;
		if (Settings == nullptr || Settings->Codecs.Num() != 1)
		{
			continue;
		}

		UAnimBoneCompressionCodec_ACLDatabase* DatabaseCodec = Cast<UAnimBoneCompressionCodec_ACLDatabase>(Settings->Codecs[0]);
		if (DatabaseCodec != nullptr && DatabaseCodec->DatabaseAsset == this)
		{
			ReferencingAnimSequences.Add(AnimSeq);
		}
	}

	// Sort our anim sequences by path name to ensure predictable and consistent results
	struct FCompareObjectNames
	{
		FORCEINLINE bool operator()(const UAnimSequence& Lhs, const UAnimSequence& Rhs) const
		{
			return Lhs.GetPathName().Compare(Rhs.GetPathName()) < 0;
		}
	};
	ReferencingAnimSequences.Sort(FCompareObjectNames());

	// Check if the list is any different to avoid marking as dirty if we aren't
	const int32 NumSeqs = ReferencingAnimSequences.Num();
	bool bIsDirty = false;
	if (AnimSequences.Num() == NumSeqs)
	{
		for (int32 SeqIdx = 0; SeqIdx < NumSeqs; ++SeqIdx)
		{
			if (AnimSequences[SeqIdx] != ReferencingAnimSequences[SeqIdx])
			{
				// Sorted arrays do not match, we are dirty
				bIsDirty = true;
				break;
			}
		}
	}
	else
	{
		// Count differs, dirty for sure
		bIsDirty = true;
	}

	if (bIsDirty)
	{
		// Swap our content since we are new
		Swap(AnimSequences, ReferencingAnimSequences);

		MarkPackageDirty();

		// If we were previewing, update our database
		if (PreviewDatabaseStreamer)
		{
			UpdatePreviewState(true);
		}
	}

	return bIsDirty;
}
#endif

static void FailAllRequests(const TArray<FFidelityChangeRequest>& Requests)
{
	for (const FFidelityChangeRequest& Request : Requests)
	{
		if (Request.Result != nullptr)
		{
			*Request.Result = ACLVisualFidelityChangeResult::Failed;
		}
	}
}

void UAnimationCompressionLibraryDatabase::BeginDestroy()
{
	Super::BeginDestroy();

	// Remove our ticker if we have one
	if (FidelityUpdateTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(FidelityUpdateTickerHandle);
		FidelityUpdateTickerHandle.Reset();
	}

	// If we have visual fidelity changes in progress, cancel them
	FailAllRequests(FidelityChangeRequests);
	FidelityChangeRequests.Empty();

	if (DatabaseStreamer)
	{
		// Wait for any pending IO requests
		UEDatabaseStreamer* Streamer = (UEDatabaseStreamer*)DatabaseStreamer.Release();
		Streamer->WaitForStreamingToComplete();

		// Reset our context to make sure it no longer references the streamer
		DatabaseContext.reset();

		// Free our streamer, it is no longer needed
		delete Streamer;
	}

#if WITH_EDITORONLY_DATA
	if (PreviewDatabaseStreamer)
	{
		// Reset our context to make sure it no longer references the streamer
		DatabaseContext.reset();

		// Free our streamer, it is no longer needed
		PreviewDatabaseStreamer.Reset();
	}
#endif

	// Manually run the destructor since the container is opaque
	DatabaseContext.~database_context<UEDefaultDatabaseSettings>();
}

void UAnimationCompressionLibraryDatabase::PostLoad()
{
	Super::PostLoad();

	if (CookedCompressedBytes.Num() != 0)
	{
		const acl::compressed_database* CompressedDatabase = acl::make_compressed_database(CookedCompressedBytes.GetData());
		check(CompressedDatabase != nullptr && CompressedDatabase->is_valid(false).empty());

		DatabaseStreamer = MakeUnique<UEDatabaseStreamer>(*CompressedDatabase, CookedBulkData);

		const bool ContextInitResult = DatabaseContext.initialize(ACLAllocatorImpl, *CompressedDatabase, *DatabaseStreamer, *DatabaseStreamer);
		checkf(ContextInitResult, TEXT("ACL failed to initialize the database context"));
	}

	if (!GIsEditor)
	{
		// When launching outside the editor, we set the default visual fidelity
		// In the editor, we use the preview fidelity instead
		SetVisualFidelity(DefaultVisualFidelity);
	}
}

void UAnimationCompressionLibraryDatabase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	bool bCooked = Ar.IsCooking();
	Ar << bCooked;

	// Serialize the cooked bulk data ONLY if we are cooking.
	if (bCooked)
	{
		CookedBulkData.SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
		CookedBulkData.Serialize(Ar, this, INDEX_NONE, false);
	}
	else if (Ar.IsCountingMemory())
	{
		// When counting memory, also track the streamed in data.
		if (DatabaseContext.is_initialized() && DatabaseStreamer)
		{
			const uint8* MediumTierData = DatabaseStreamer->get_bulk_data(acl::quality_tier::medium_importance);
			if (MediumTierData != nullptr)
			{
				const uint32 MediumTierSize = DatabaseContext.get_compressed_database()->get_bulk_data_size(acl::quality_tier::medium_importance);

				// Avoid touching the actual tier data, use a temporary array since the content itself doesn't matter
				TArray<uint8> Tmp;
				Tmp.Empty(MediumTierSize);
				Tmp.AddZeroed(MediumTierSize);

				Ar << Tmp;
			}

			const uint8* LowestTierData = DatabaseStreamer->get_bulk_data(acl::quality_tier::lowest_importance);
			if (LowestTierData != nullptr)
			{
				const uint32 LowestTierSize = DatabaseContext.get_compressed_database()->get_bulk_data_size(acl::quality_tier::lowest_importance);

				// Avoid touching the actual tier data, use a temporary array since the content itself doesn't matter
				TArray<uint8> Tmp;
				Tmp.Empty(LowestTierSize);
				Tmp.AddZeroed(LowestTierSize);

				Ar << Tmp;
			}
		}
	}
}

void UAnimationCompressionLibraryDatabase::SetVisualFidelity(ACLVisualFidelity VisualFidelity)
{
	SetVisualFidelityImpl(VisualFidelity, nullptr);
}

uint32 UAnimationCompressionLibraryDatabase::SetVisualFidelityImpl(ACLVisualFidelity VisualFidelity, ACLVisualFidelityChangeResult* OutResult)
{
	// Must execute on the main thread but must do so while animations aren't updating
	check(IsInGameThread());

#if WITH_EDITORONLY_DATA
	if (!DatabaseContext.is_initialized())
	{
		// We are in the editor and we aren't previewing yet which means the highest quality is showing, build the database now
		// so we can properly preview
		UpdatePreviewState(true);
	}
#endif

	check(DatabaseContext.is_initialized());

	const bool bIsFirstRequest = FidelityChangeRequests.Num() == 0;
	const ACLVisualFidelity FinalEffectiveFidelity = bIsFirstRequest ? CurrentVisualFidelity : FidelityChangeRequests.Last().Fidelity;

	// If the desired visual fidelity matches the final effective fidelity once all change requests have completed, we can ignore it
	if (VisualFidelity == FinalEffectiveFidelity)
	{
		if (OutResult != nullptr)
		{
			*OutResult = ACLVisualFidelityChangeResult::Completed;
		}

		return ~0U;
	}

	UE_LOGF(LogAnimationCompression, Log, "ACL database is requesting visual fidelity %ls [%ls]", VisualFidelityToString(VisualFidelity), *GetPathName());

	const uint32 RequestID = NextFidelityChangeRequestID++;

	// Add our change requests
	// To simplify handling, change requests only transition from one fidelity level to the next closest
	// Large changes are split into simpler steps
	switch (FinalEffectiveFidelity)
	{
	case ACLVisualFidelity::Highest:
		switch (VisualFidelity)
		{
		case ACLVisualFidelity::Medium:
			// From highest to medium we need a single change
			FidelityChangeRequests.Add({ OutResult, RequestID, ACLVisualFidelity::Medium, false });
			break;
		case ACLVisualFidelity::Lowest:
			// From highest to lowest we need two changes
			FidelityChangeRequests.Add({ nullptr, RequestID, ACLVisualFidelity::Medium, false });
			FidelityChangeRequests.Add({ OutResult, RequestID, ACLVisualFidelity::Lowest, false });
			break;
		default:
			checkf(false, TEXT("Unexpected visual fidelity value"));
			break;
		}
		break;
	case ACLVisualFidelity::Medium:
		switch (VisualFidelity)
		{
		case ACLVisualFidelity::Highest:
			// From medium to highest we need a single change
			FidelityChangeRequests.Add({ OutResult, RequestID, ACLVisualFidelity::Highest, false });
			break;
		case ACLVisualFidelity::Lowest:
			// From medium to lowest we need a single change
			FidelityChangeRequests.Add({ OutResult, RequestID, ACLVisualFidelity::Lowest, false });
			break;
		default:
			checkf(false, TEXT("Unexpected visual fidelity value"));
			break;
		}
		break;
	case ACLVisualFidelity::Lowest:
		switch (VisualFidelity)
		{
		case ACLVisualFidelity::Highest:
			// From lowest to highest we need two changes
			FidelityChangeRequests.Add({ nullptr, RequestID, ACLVisualFidelity::Medium, false });
			FidelityChangeRequests.Add({ OutResult, RequestID, ACLVisualFidelity::Highest, false });
			break;
		case ACLVisualFidelity::Medium:
			// From lowest to medium we need a single change
			FidelityChangeRequests.Add({ OutResult, RequestID, ACLVisualFidelity::Medium, false });
			break;
		default:
			checkf(false, TEXT("Unexpected visual fidelity value"));
			break;
		}
		break;
	}

	// If this is the first request, queue our ticker so we can start tracking our changes
	if (bIsFirstRequest)
	{
		check(!FidelityUpdateTickerHandle.IsValid());
		auto UpdateVisualFidelity = [this](float DeltaTime) { return UpdateVisualFidelityTicker(DeltaTime); };
		FidelityUpdateTickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("ACLDBStreamOut"), 0.0F, UpdateVisualFidelity);
	}

	return RequestID;
}

void UAnimationCompressionLibraryDatabase::CancelVisualFidelityRequestImpl(uint32 RequestID)
{
	// Must execute on the main thread
	check(IsInGameThread());

	// Nothing to do if the request ID isn't valid
	if (RequestID == ~0U)
	{
		return;
	}

	// Look for our request ID and cancel what we can
	// Iterate in reverse order so we can remove elements as we go
	for (int32 RequestIndex = FidelityChangeRequests.Num() - 1; RequestIndex >= 0; --RequestIndex)
	{
		FFidelityChangeRequest& Request = FidelityChangeRequests[RequestIndex];
		if (Request.RequestID != RequestID)
		{
			// Not our request, skip it
			continue;
		}

		// Found our request
		if (Request.bIsInProgress)
		{
			// Our request is in progress but we can't cancel it
			// The request will attempt to complete but the result pointer might no longer be valid then, clear it
			Request.Result = nullptr;
		}
		else
		{
			// Our request hasn't started yet, remove it
			FidelityChangeRequests.RemoveAt(RequestIndex);
		}
	}
}

static void LogRequestResult(const UAnimationCompressionLibraryDatabase& Database, acl::database_stream_request_result Result)
{
	switch (Result)
	{
	default:
		UE_LOGF(LogAnimationCompression, Log, "Unknown ACL database stream request result: %u [%ls]", uint32(Result), *Database.GetPathName());
		break;
	case acl::database_stream_request_result::done:
		UE_LOGF(LogAnimationCompression, Log, "ACL database streaming is done [%ls]", *Database.GetPathName());
		break;
	case acl::database_stream_request_result::dispatched:
		break;
	case acl::database_stream_request_result::streaming_in_progress:
		UE_LOGF(LogAnimationCompression, Log, "ACL database streaming is already in progress [%ls]", *Database.GetPathName());
		break;
	case acl::database_stream_request_result::context_not_initialized:
		UE_LOGF(LogAnimationCompression, Log, "ACL database context not initialized [%ls]", *Database.GetPathName());
		break;
	case acl::database_stream_request_result::invalid_database_tier:
		UE_LOGF(LogAnimationCompression, Log, "Specified ACL database tier is invalid [%ls]", *Database.GetPathName());
		break;
	case acl::database_stream_request_result::no_free_streaming_requests:
		UE_LOGF(LogAnimationCompression, Log, "Failed to find a free ACL database streaming request [%ls]", *Database.GetPathName());
		break;
	}
}

static uint32 CalculateNumChunksToStream(const acl::compressed_database& Database, uint32 MaxStreamRequestSizeKB)
{
	if (MaxStreamRequestSizeKB == 0)
	{
		return ~0U;
	}

	const uint32 MaxStreamRequestSize = MaxStreamRequestSizeKB * 1024;
	const uint32 MaxChunkSize = Database.get_max_chunk_size();
	return FMath::Max<uint32>(MaxStreamRequestSize / MaxChunkSize, 1);	// Must stream at least one chunk
}

/**
* Visual fidelity is updated through this latent ticker. Depending on the current database fidelity,
* we determine whether we need to stream in/out or do nothing.
* If a fidelity change is ongoing and a new value is requested, it will be queued. Subsequent changes
* at the same fidelity level simply update the parameters and do nothing.
* This greatly simplifies the level of control that ACL offers. A user can simply dictate the desired quality level
* and let the plugin determine what to do.
*/
bool UAnimationCompressionLibraryDatabase::UpdateVisualFidelityTicker(float DeltaTime)
{
	check(DatabaseContext.is_initialized());

	const uint32 NumChunksToStream = CalculateNumChunksToStream(*DatabaseContext.get_compressed_database(), MaxStreamRequestSizeKB);

	while (FidelityChangeRequests.Num() != 0)
	{
		FFidelityChangeRequest& Request = FidelityChangeRequests[0];
		check(Request.Fidelity != CurrentVisualFidelity);

		bool bIsRequestCompleted = false;
		acl::database_stream_request_result Result = acl::database_stream_request_result::context_not_initialized;

		switch (CurrentVisualFidelity)
		{
		default:
		case ACLVisualFidelity::Highest:
			// We have highest fidelity
			// The medium/lowest importance tiers are already streamed in

			if (!ensureMsgf(Request.Fidelity == ACLVisualFidelity::Medium, TEXT("Unexpected visual fidelity value")))
			{
				// Something wrong happened, ignore all change requests
				FailAllRequests(FidelityChangeRequests);
				FidelityChangeRequests.Empty();
			}
			else
			{
				// To reach medium quality, we need to stream out our lowest importance tier
				Result = DatabaseContext.stream_out(acl::quality_tier::lowest_importance, NumChunksToStream);
				LogRequestResult(*this, Result);

				switch (Result)
				{
				case acl::database_stream_request_result::done:
					// We are done
					bIsRequestCompleted = true;
					break;
				case acl::database_stream_request_result::dispatched:
					// Streaming request has been dispatched, done for now
					Request.bIsInProgress = true;
					break;
				case acl::database_stream_request_result::streaming_in_progress:
					// Streaming is already in progress, done for now
					checkf(Request.bIsInProgress, TEXT("Expected request to already be in progress"));
					break;
				default:
					// Something wrong happened, ignore all change requests
					checkf(false, TEXT("Something unexpected happened while ACL is streaming"));
					FailAllRequests(FidelityChangeRequests);
					FidelityChangeRequests.Empty();
					break;
				}
			}

			// Done
			break;
		case ACLVisualFidelity::Medium:
			// We have medium fidelity
			// The medium importance tier is already streamed in

			checkf(Request.Fidelity == ACLVisualFidelity::Highest || Request.Fidelity == ACLVisualFidelity::Lowest, TEXT("Unexpected visual fidelity value"));
			if (Request.Fidelity != ACLVisualFidelity::Highest && Request.Fidelity != ACLVisualFidelity::Lowest)
			{
				// Something wrong happened, ignore all change requests
				FailAllRequests(FidelityChangeRequests);
				FidelityChangeRequests.Empty();
			}
			else
			{
				if (Request.Fidelity == ACLVisualFidelity::Highest)
				{
					// To reach highest quality, we need to stream in our lowest importance tier
					Result = DatabaseContext.stream_in(acl::quality_tier::lowest_importance, NumChunksToStream);
				}
				else
				{
					// To reach lowest quality, we need to stream out our medium importance tier
					Result = DatabaseContext.stream_out(acl::quality_tier::medium_importance, NumChunksToStream);
				}

				LogRequestResult(*this, Result);

				switch (Result)
				{
				case acl::database_stream_request_result::done:
					// We are done
					bIsRequestCompleted = true;
					break;
				case acl::database_stream_request_result::dispatched:
					// Streaming request has been dispatched, done for now
					Request.bIsInProgress = true;
					break;
				case acl::database_stream_request_result::streaming_in_progress:
					// Streaming is already in progress, done for now
					checkf(Request.bIsInProgress, TEXT("Expected request to already be in progress"));
					break;
				default:
					// Something wrong happened, ignore all change requests
					checkf(false, TEXT("Something unexpected happened while ACL is streaming"));
					FailAllRequests(FidelityChangeRequests);
					FidelityChangeRequests.Empty();
					break;
				}
			}

			// Done
			break;
		case ACLVisualFidelity::Lowest:
			// We have lowest fidelity
			// Nothing is currently streamed in

			if (!ensureMsgf(Request.Fidelity == ACLVisualFidelity::Medium, TEXT("Unexpected visual fidelity value")))
			{
				// Something wrong happened, ignore all change requests
				FailAllRequests(FidelityChangeRequests);
				FidelityChangeRequests.Empty();
			}
			else
			{
				// To reach medium quality, we need to stream in our medium importance tier
				Result = DatabaseContext.stream_in(acl::quality_tier::medium_importance, NumChunksToStream);
				LogRequestResult(*this, Result);

				switch (Result)
				{
				case acl::database_stream_request_result::done:
					// We are done
					bIsRequestCompleted = true;
					break;
				case acl::database_stream_request_result::dispatched:
					// Streaming request has been dispatched, done for now
					Request.bIsInProgress = true;
					break;
				case acl::database_stream_request_result::streaming_in_progress:
					// Streaming is already in progress, done for now
					checkf(Request.bIsInProgress, TEXT("Expected request to already be in progress"));
					break;
				default:
					// Something wrong happened, ignore all change requests
					checkf(false, TEXT("Something unexpected happened while ACL is streaming"));
					FailAllRequests(FidelityChangeRequests);
					FidelityChangeRequests.Empty();
					break;
				}
			}

			// Done
			break;
		}

		// If we completed this request, consume it and loop again otherwise stop
		if (bIsRequestCompleted)
		{
			UE_LOGF(LogAnimationCompression, Log, "ACL database is changing visual fidelity from %ls to %ls [%ls]", VisualFidelityToString(CurrentVisualFidelity), VisualFidelityToString(Request.Fidelity), *GetPathName());

#if WITH_EDITORONLY_DATA
			// Make sure our preview fidelity matches what just changed
			PreviewVisualFidelity = Request.Fidelity;
#endif

			CurrentVisualFidelity = Request.Fidelity;
			if (Request.Result != nullptr)
			{
				*Request.Result = ACLVisualFidelityChangeResult::Completed;
			}

			FidelityChangeRequests.RemoveAt(0);
		}
		else
		{
			break;
		}
	}

	const bool bHasMoreRequests = FidelityChangeRequests.Num() != 0;
	if (!bHasMoreRequests)
	{
		// Clear our ticker handle, no longer needed
		FidelityUpdateTickerHandle.Reset();
	}

	return bHasMoreRequests;	// We need to fire again if we have more pending requests
}

class FSetDatabaseVisualFidelityAction final : public FPendingLatentAction
{
public:
	FSetDatabaseVisualFidelityAction(UAnimationCompressionLibraryDatabase* DatabaseAsset_, ACLVisualFidelity VisualFidelity_, ACLVisualFidelityChangeResult& OutResult_, const FLatentActionInfo& LatentInfo_)
		: DatabaseAsset(DatabaseAsset_)
		, VisualFidelity(VisualFidelity_)
		, OutResult(OutResult_)
		, LatentInfo(LatentInfo_)
		, bIsDispatched(false)
		, RequestID(~0U)
	{
	}

	virtual void UpdateOperation(FLatentResponse& Response) override
	{
		// Must execute on the main thread
		check(IsInGameThread());

		if (DatabaseAsset == nullptr)
		{
			// No valid asset provided
			OutResult = ACLVisualFidelityChangeResult::Failed;
			Response.FinishAndTriggerIf(true, LatentInfo.ExecutionFunction, LatentInfo.Linkage, LatentInfo.CallbackTarget);
			return;
		}

		bool bDispatchedNow = false;
		if (!bIsDispatched)
		{
			// Fire off our change request, will complete right away if no change is requested, otherwise we just dispatch
			bIsDispatched = true;
			bDispatchedNow = true;
			OutResult = ACLVisualFidelityChangeResult::Dispatched;
			RequestID = DatabaseAsset->SetVisualFidelityImpl(VisualFidelity, &OutResult);
		}

		// We are done once our result is set
		const bool bIsDone = OutResult != ACLVisualFidelityChangeResult::Dispatched;
		if (bIsDone)
		{
			Response.FinishAndTriggerIf(bIsDone, LatentInfo.ExecutionFunction, LatentInfo.Linkage, LatentInfo.CallbackTarget);
		}
		else if (bDispatchedNow)
		{
			Response.TriggerLink(LatentInfo.ExecutionFunction, LatentInfo.Linkage, LatentInfo.CallbackTarget);
		}
	}

	virtual void NotifyObjectDestroyed() override
	{
		// Nothing we can do
	}

	virtual void NotifyActionAborted() override
	{
		// Our request is being aborted, cancel what we can
		if (bIsDispatched && RequestID != ~0U)
		{
			OutResult = ACLVisualFidelityChangeResult::Failed;
			DatabaseAsset->CancelVisualFidelityRequestImpl(RequestID);
		}
	}

#if WITH_EDITOR
	virtual FString GetDescription() const override
	{
		return TEXT("Updating ACL database visual fidelity");
	}
#endif

private:
	UAnimationCompressionLibraryDatabase* DatabaseAsset;

	ACLVisualFidelity VisualFidelity;
	ACLVisualFidelityChangeResult& OutResult;

	FLatentActionInfo LatentInfo;
	bool bIsDispatched;
	uint32 RequestID;
};

void UAnimationCompressionLibraryDatabase::SetVisualFidelity(UObject* WorldContextObject, FLatentActionInfo LatentInfo, UAnimationCompressionLibraryDatabase* DatabaseAsset, ACLVisualFidelityChangeResult& Result, ACLVisualFidelity VisualFidelity)
{
	// Must execute on the main thread but must do so while animations aren't updating
	check(IsInGameThread());

	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		FLatentActionManager& LatentManager = World->GetLatentActionManager();
		if (LatentManager.FindExistingAction<FSetDatabaseVisualFidelityAction>(LatentInfo.CallbackTarget, LatentInfo.UUID) == nullptr)
		{
			FSetDatabaseVisualFidelityAction* NewAction = new FSetDatabaseVisualFidelityAction(DatabaseAsset, VisualFidelity, Result, LatentInfo);
			LatentManager.AddNewAction(LatentInfo.CallbackTarget, LatentInfo.UUID, NewAction);
		}
	}
}

ACLVisualFidelity UAnimationCompressionLibraryDatabase::GetVisualFidelity(UAnimationCompressionLibraryDatabase* DatabaseAsset)
{
	checkf(DatabaseAsset != nullptr, TEXT("Cannot query null ACL database asset"));
	return DatabaseAsset != nullptr ? DatabaseAsset->CurrentVisualFidelity : ACLVisualFidelity::Lowest;
}
