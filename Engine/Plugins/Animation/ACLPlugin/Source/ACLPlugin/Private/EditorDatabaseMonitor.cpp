// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2021 Nicholas Frechette. All Rights Reserved.

#include "EditorDatabaseMonitor.h"

#if WITH_EDITORONLY_DATA

#include "AnimationCompressionLibraryDatabase.h"

#include "Containers/Ticker.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace EditorDatabaseMonitor
{
	static FTSTicker::FDelegateHandle MonitorTickerHandle;
	static TArray<TWeakObjectPtr<UAnimationCompressionLibraryDatabase>> DirtyDatabases;
	static FCriticalSection DirtyDatabasesCS;

	static bool MonitorTicker(float DeltaTime)
	{
		// Copy our array with a quick swap to avoid holding the lock too long
		TArray<TWeakObjectPtr<UAnimationCompressionLibraryDatabase>> DirtyDatabasesTmp;
		{
			FScopeLock Lock(&DirtyDatabasesCS);
			Swap(DirtyDatabases, DirtyDatabasesTmp);
		}

		// Iterate over our dirty databases and refresh any stale mappings we might have
		for (TWeakObjectPtr<UAnimationCompressionLibraryDatabase>& DatabasePtr : DirtyDatabasesTmp)
		{
			if (UAnimationCompressionLibraryDatabase* Database = DatabasePtr.Get())
			{
				Database->UpdateReferencingAnimSequenceList();
			}
		}

		const bool bFireTickerAgainAfterDelay = true;
		return bFireTickerAgainAfterDelay;
	}

	void RegisterMonitor()
	{
		if (MonitorTickerHandle.IsValid())
		{
			return;	// Already registered
		}

		// Tick every 300ms
		const float TickerDelay = 0.3F;

		MonitorTickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("ACLEditorDatabaseMonitor"), TickerDelay, MonitorTicker);
	}

	void UnregisterMonitor()
	{
		if (MonitorTickerHandle.IsValid())
		{
			FTSTicker::RemoveTicker(MonitorTickerHandle);
			MonitorTickerHandle.Reset();
		}
	}

	void MarkDirty(UAnimationCompressionLibraryDatabase* Database)
	{
		if (Database == nullptr)
		{
			return;	// Nothing to do
		}

		// Add our database, we'll process it later
		FScopeLock Lock(&DirtyDatabasesCS);
		DirtyDatabases.AddUnique(Database);
	}
}

#endif
