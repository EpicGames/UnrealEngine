// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2021 Nicholas Frechette. All Rights Reserved.

#include "CoreMinimal.h"

#if WITH_EDITORONLY_DATA

class UAnimationCompressionLibraryDatabase;

/** A central database monitor that ensures database instances have their mappings up to date. */
namespace EditorDatabaseMonitor
{
	void RegisterMonitor();
	void UnregisterMonitor();

	void MarkDirty(UAnimationCompressionLibraryDatabase* Database);
}

#endif
