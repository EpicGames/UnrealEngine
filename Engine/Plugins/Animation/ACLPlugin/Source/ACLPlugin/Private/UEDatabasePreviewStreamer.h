// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2021 Nicholas Frechette. All Rights Reserved.

#include "AnimationCompression.h"
#include "CoreMinimal.h"
#include "HAL/UnrealMemory.h"

#include "ACLImpl.h"

THIRD_PARTY_INCLUDES_START
#include <acl/core/compressed_database.h>
#include <acl/decompression/database/database_streamer.h>
THIRD_PARTY_INCLUDES_END


/** A simple UE preview streamer. Everything is assumed to be in memory and no real streaming is done. */
class UEDatabasePreviewStreamer final : public acl::database_streamer
{
public:
	UEDatabasePreviewStreamer(const acl::compressed_database& CompressedDatabase, const TArray<uint8>& BulkData_)
		: database_streamer(Requests, acl::k_num_database_tiers)
	{
		const uint32 BulkDataMediumSize = CompressedDatabase.get_bulk_data_size(acl::quality_tier::medium_importance);

		const uint8* BulkDataMedium = BulkData_.GetData();
		BulkData[0] = BulkDataMedium;
		BulkData[1] = BulkDataMedium + BulkDataMediumSize;
	}

	virtual bool is_initialized() const override { return true; }

	virtual const uint8_t* get_bulk_data(acl::quality_tier Tier) const override
	{
		checkf(Tier != acl::quality_tier::highest_importance, TEXT("Unexpected quality tier"));
		const uint32 TierIndex = uint32(Tier) - 1;
		return BulkData[TierIndex];
	}

	virtual void stream_in(uint32_t Offset, uint32_t Size, bool CanAllocateBulkData, acl::quality_tier Tier, acl::streaming_request_id RequestID) override
	{
		UE_LOGF(LogAnimationCompression, Log, "ACL database bulk data is streaming in!");
		complete(RequestID);
	}

	virtual void stream_out(uint32_t Offset, uint32_t Size, bool CanDeallocateBulkData, acl::quality_tier Tier, acl::streaming_request_id RequestID) override
	{
		UE_LOGF(LogAnimationCompression, Log, "ACL database bulk data is streaming out!");
		complete(RequestID);
	}

private:
	UEDatabasePreviewStreamer(const UEDatabasePreviewStreamer&) = delete;
	UEDatabasePreviewStreamer& operator=(const UEDatabasePreviewStreamer&) = delete;

	const uint8* BulkData[acl::k_num_database_tiers];

	acl::streaming_request Requests[acl::k_num_database_tiers];	// One request per tier is enough
};
