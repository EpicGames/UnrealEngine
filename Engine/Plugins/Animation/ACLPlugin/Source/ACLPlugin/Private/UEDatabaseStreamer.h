// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "AnimationCompression.h"
#include "CoreMinimal.h"
#include "HAL/UnrealMemory.h"

#include "ACLImpl.h"
#include "Serialization/BulkData.h"
#include "ProfilingDebugging/IoStoreTrace.h"

THIRD_PARTY_INCLUDES_START
#include <acl/decompression/database/database_streamer.h>
THIRD_PARTY_INCLUDES_END

// UE 4.25 doesn't expose its virtual memory management, see FPlatformMemory::FPlatformVirtualMemoryBlock
#define WITH_VMEM_MANAGEMENT 0

/** A simple async UE streamer. Memory is allocated on the first stream in request and deallocated on the last stream out request. */
class UEDatabaseStreamer final : public acl::database_streamer
{
public:
	UEDatabaseStreamer(const acl::compressed_database& CompressedDatabase, FByteBulkData& StreamableBulkData_)
		: database_streamer(Requests, acl::k_num_database_tiers)
		, StreamableBulkData(StreamableBulkData_)
		, PendingIORequest(nullptr)
	{
		BulkData[0] = BulkData[1] = nullptr;

		const uint32 BulkDataMediumSize = CompressedDatabase.get_bulk_data_size(acl::quality_tier::medium_importance);
		const uint32 BulkDataLowSize = CompressedDatabase.get_bulk_data_size(acl::quality_tier::lowest_importance);

		BulkDataSize[0] = BulkDataMediumSize;
		BulkDataSize[1] = BulkDataLowSize;

#if WITH_VMEM_MANAGEMENT
		// Allocate but don't commit the memory until we need it
		// TODO: Commit right away if requested
		
		StreamedBulkDataBlock = FPlatformMemory::FPlatformVirtualMemoryBlock::AllocateVirtual(BulkDataTotalSize);
		BulkDataPtr = static_cast<uint8*>(StreamedBulkDataBlock.GetVirtualPointer());
		bIsBulkDataCommited = false;
#endif
	}

	virtual ~UEDatabaseStreamer()
	{
		// If we have a stream in request, wait for it to complete and clear it
		WaitForStreamingToComplete();

#if WITH_VMEM_MANAGEMENT
		StreamedBulkDataBlock.FreeVirtual();
#else
		delete[] BulkData[0];
		delete[] BulkData[1];
#endif
	}

	virtual bool is_initialized() const override { return true; }

	virtual const uint8_t* get_bulk_data(acl::quality_tier Tier) const override
	{
		checkf(Tier != acl::quality_tier::highest_importance, TEXT("Unexpected quality tier"));
		const uint32 TierIndex = uint32(Tier) - 1;
		return BulkData[TierIndex];
	}

	virtual void stream_in(uint32_t Offset, uint32_t Size, bool CanAllocateBulkData, acl::quality_tier Tier, acl::streaming_request_id RequestID) override
	{
		const uint32 TierIndex = uint32(Tier) - 1;
		const uint32 TierBulkDataSize = BulkDataSize[TierIndex];

		checkf(Offset < TierBulkDataSize, TEXT("Steam offset is outside of the bulk data range"));
		checkf(Size <= TierBulkDataSize, TEXT("Stream size is larger than the bulk data size"));
		checkf(uint64(Offset) + uint64(Size) <= uint64(TierBulkDataSize), TEXT("Streaming request is outside of the bulk data range"));
		checkf(Tier != acl::quality_tier::highest_importance, TEXT("Unexpected quality tier"));

		// If we already did a stream in request, wait for it to complete and clear it
		WaitForStreamingToComplete();

		FBulkDataIORequestCallBack AsyncFileCallBack = [this, RequestID](bool bWasCancelled, IBulkDataIORequest* Req)
		{
			UE_LOGF(LogAnimationCompression, Log, "ACL completed the stream in request!");

			// Tell ACL whether the streaming request was a success or not, this is thread safe
			if (bWasCancelled)
				this->cancel(RequestID);
			else
				this->complete(RequestID);
		};

		UE_LOGF(LogAnimationCompression, Log, "ACL starting a new stream in request!");

		// Allocate our bulk data buffer on the first stream in request
		if (CanAllocateBulkData)
		{
			UE_LOGF(LogAnimationCompression, Log, "ACL is allocating the database bulk data!");

#if WITH_VMEM_MANAGEMENT
			check(!bIsBulkDataCommited);
			StreamedBulkDataBlock.Commit();
			bIsBulkDataCommited = true;
#else
			check(BulkData[TierIndex] == nullptr);
			BulkData[TierIndex] = new uint8[TierBulkDataSize];
#endif
		}

		// Fire off our async streaming request
		const uint32 BulkDataTierStartOffset = Tier == acl::quality_tier::medium_importance ? 0 : BulkDataSize[0];
		const uint64 BulkDataReadOffset = BulkDataTierStartOffset + Offset;

		uint8* StreamedBulkDataPtr = BulkData[TierIndex];
		uint8* DestBulkDataPtr = StreamedBulkDataPtr + Offset;

		TRACE_IOSTORE_METADATA_SCOPE_TAG("ACL");
		PendingIORequest = StreamableBulkData.CreateStreamingRequest(BulkDataReadOffset, Size, AIOP_Low, &AsyncFileCallBack, DestBulkDataPtr);
		if (PendingIORequest == nullptr)
		{
			UE_LOGF(LogAnimationCompression, Warning, "ACL failed to initiate database stream in request!");
			cancel(RequestID);
		}
	}

	virtual void stream_out(uint32_t Offset, uint32_t Size, bool CanDeallocateBulkData, acl::quality_tier Tier, acl::streaming_request_id RequestID) override
	{
		const uint32 TierIndex = uint32(Tier) - 1;
		const uint32 TierBulkDataSize = BulkDataSize[TierIndex];

		checkf(Offset < TierBulkDataSize, TEXT("Steam offset is outside of the bulk data range"));
		checkf(Size <= TierBulkDataSize, TEXT("Stream size is larger than the bulk data size"));
		checkf(uint64(Offset) + uint64(Size) <= uint64(TierBulkDataSize), TEXT("Streaming request is outside of the bulk data range"));
		checkf(Tier != acl::quality_tier::highest_importance, TEXT("Unexpected quality tier"));

		// If we already did a stream in request, wait for it to complete and clear it
		WaitForStreamingToComplete();

		UE_LOGF(LogAnimationCompression, Log, "ACL is streaming out a database!");

		// Free our bulk data on the last stream out request
		if (CanDeallocateBulkData)
		{
			// TODO: Make this optional?
			UE_LOGF(LogAnimationCompression, Log, "ACL is deallocating the database bulk data!");

#if WITH_VMEM_MANAGEMENT
			check(bIsBulkDataCommited);
			StreamedBulkDataBlock.Decommit();
			bIsBulkDataCommited = false;
#else
			check(BulkData[TierIndex] != nullptr);
			delete[] BulkData[TierIndex];
			BulkData[TierIndex] = nullptr;
#endif
		}

		// Notify ACL that we streamed out the data, this is not thread safe and cannot run while animations are decompressing
		complete(RequestID);
	}

	void WaitForStreamingToComplete()
	{
		if (PendingIORequest != nullptr)
		{
			verify(PendingIORequest->WaitCompletion());
			delete PendingIORequest;
			PendingIORequest = nullptr;
		}
	}

private:
	UEDatabaseStreamer(const UEDatabaseStreamer&) = delete;
	UEDatabaseStreamer& operator=(const UEDatabaseStreamer&) = delete;

	FByteBulkData& StreamableBulkData;
	uint8* BulkData[acl::k_num_database_tiers];
	uint32 BulkDataSize[acl::k_num_database_tiers];

	IBulkDataIORequest* PendingIORequest;

	acl::streaming_request Requests[acl::k_num_database_tiers];	// One request per tier is enough

#if WITH_VMEM_MANAGEMENT
	FPlatformMemory::FPlatformVirtualMemoryBlock StreamedBulkDataBlock;
	bool bIsBulkDataCommited;
#endif
};
