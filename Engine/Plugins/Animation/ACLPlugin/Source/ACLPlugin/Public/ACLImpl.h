// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/UnrealMemory.h"
#include "UObject/ObjectMacros.h"
#include "Runtime/Launch/Resources/Version.h"

#if DO_GUARD_SLOW
	// ACL has a lot of asserts, only enabled in Debug
	// This decision has been made because the extensive asserts add considerable overhead and most
	// developers use a Development configuration for the editor. The ACL library runs extensive
	// unit and regression tests on a very large number of clips which minimizes the risk of
	// having a legitimate assert fire.

	#define ACL_ON_ASSERT_CUSTOM

	// Override the string format specifier for UE
	#define RTM_ASSERT_STRING_FORMAT_SPECIFIER "%hs"
	#define SJSON_ASSERT_STRING_FORMAT_SPECIFIER "%hs"
	#define ACL_ASSERT_STRING_FORMAT_SPECIFIER "%hs"

	#define ACL_ASSERT(expression, format, ...) checkf(expression, TEXT(format), ##__VA_ARGS__)
#endif

//////////////////////////////////////////////////////////////////////////
// [Bind pose stripping]
// Bind pose stripping allows ACL to remove the redundant bind pose often present in the compressed
// data of joints that aren't animated. For this to work properly, we must have access to the bind
// pose during decompression which is not accessible in UE 5.0 and earlier.
//
// When stripping is disabled, here is what happens:
//     Non-additive anim sequence
//         - During compression, sub-tracks have their default value set to the identity.
//           Sub-tracks equal to the identity will be stripped (uncommon).
//         - When we decompress a whole pose, sub-tracks equal to the identity are processed
//           and the identity is written out.
//         - When we decompress a single bone, sub-tracks equal to the identity are processed
//           and the identity is written out.
// 
//     Additive anim sequence
//         - During compression, sub-tracks have their default value set to the additive identity.
//           Sub-tracks equal to the identity will be stripped (common).
//         - When we decompress a whole pose, sub-tracks equal to the additive identity are skipped entirely.
//           This is possible because the output pose is initialized with the additive identity before decompression.
//         - When we decompress a single bone, sub-tracks equal to the identity are processed
//           and the identity is written out.
// 
// When stripping is enabled, here is what happens:
//     Non-additive anim sequence
//         - During compression, sub-tracks have their default value set to the bind pose.
//           Sub-tracks equal to the bind pose will be stripped (common).
//         - When we decompress a whole pose, sub-tracks equal to the bind pose are skipped entirely.
//           This is possible because the output pose is initialized with the bind pose before decompression.
//         - When we decompress a single bone, sub-tracks equal to the bind pose are processed
//           and the bind pose is written out.
// 
//     Additive anim sequence
//         - Additive sequences do not have a traditional bind pose, instead their default pose is
//           the additive identity. As such, ACL always does stripping by default for this scenario.
//           The behavior is identical as above.
//////////////////////////////////////////////////////////////////////////

//////////////////////////////////////////////////////////////////////////
// [Keyframe stripping]
// 
// ACL can strip keyframes that contribute the least amount of error.
// This is controlled through two different variables which can be used on their own or together:
//     * proportion: the minimum amount (percentage) of keyframes to strip
//     * threshold: the threshold below which keyframes are stripped
// 
// In order to support this per platform, the plugin requires the target platform to be exposed in
// a few codec API functions. Support for this has been added in UE 5.1.
//////////////////////////////////////////////////////////////////////////

THIRD_PARTY_INCLUDES_START
#include <acl/core/error.h>
#include <acl/core/iallocator.h>
#include <acl/decompression/decompress.h>

#include <rtm/quatf.h>
#include <rtm/vector4f.h>
#include <rtm/qvvf.h>
THIRD_PARTY_INCLUDES_END

#if WITH_EDITOR
THIRD_PARTY_INCLUDES_START
#include <acl/compression/track_array.h>
#include <acl/compression/compression_level.h>
THIRD_PARTY_INCLUDES_END
#endif

#include "ACLImpl.generated.h"

/** The ACL allocator implementation simply forwards to the default heap allocator. */
class ACLAllocator final : public acl::iallocator
{
public:
	virtual void* allocate(size_t size, size_t alignment = acl::iallocator::k_default_alignment)
	{
		return FMemory::Malloc(size, alignment);
	}

	virtual void deallocate(void* ptr, size_t size)
	{
		FMemory::Free(ptr);
	}
};

extern ACLPLUGIN_API ACLAllocator ACLAllocatorImpl;

/** RTM <-> UE conversion utilities and aliases. */
inline FQuat RTM_SIMD_CALL UEQuatCast(const FQuat4f& Input) { return FQuat(Input.X, Input.Y, Input.Z, Input.W); }
inline FVector RTM_SIMD_CALL UEVector3Cast(const FVector3f& Input) { return FVector(Input.X, Input.Y, Input.Z); }
inline FVector RTM_SIMD_CALL UEVector3Cast(const FVector3d& Input) { return FVector(Input.X, Input.Y, Input.Z); }

inline rtm::vector4f RTM_SIMD_CALL UEVector3ToACL(const FVector3f& Input) { return rtm::vector_set(Input.X, Input.Y, Input.Z); }
inline rtm::vector4f RTM_SIMD_CALL UEVector3ToACL(const FVector& Input) { return rtm::vector_cast(rtm::vector_set(Input.X, Input.Y, Input.Z)); }
inline FVector3f RTM_SIMD_CALL ACLVector3ToUE(rtm::vector4f_arg0 Input) { return FVector3f(rtm::vector_get_x(Input), rtm::vector_get_y(Input), rtm::vector_get_z(Input)); }

inline rtm::quatf RTM_SIMD_CALL UEQuatToACL(const FQuat4f& Input) { return rtm::quat_set(Input.X, Input.Y, Input.Z, Input.W); }
inline rtm::quatf RTM_SIMD_CALL UEQuatToACL(const FQuat& Input) { return rtm::quat_cast(rtm::quat_set(Input.X, Input.Y, Input.Z, Input.W)); }
inline FQuat4f RTM_SIMD_CALL ACLQuatToUE(rtm::quatf_arg0 Input) { return FQuat4f(rtm::quat_get_x(Input), rtm::quat_get_y(Input), rtm::quat_get_z(Input), rtm::quat_get_w(Input)); }

inline FTransform RTM_SIMD_CALL ACLTransformToUE(rtm::qvvf_arg0 Input) { return FTransform(UEQuatCast(ACLQuatToUE(Input.rotation)), UEVector3Cast(ACLVector3ToUE(Input.translation)), UEVector3Cast(ACLVector3ToUE(Input.scale))); }

using FRawAnimTrackQuat = FQuat4f;
using FRawAnimTrackVector3 = FVector3f;

/** The decompression settings used by ACL */
struct UEDefaultDecompressionSettings : public acl::default_transform_decompression_settings
{
	// Only support our latest version
	static constexpr acl::compressed_tracks_version16 version_supported() { return acl::compressed_tracks_version16::latest; }

#if UE_BUILD_SHIPPING
	// Shipping builds do not need safety checks, by then the game has been tested enough
	// Only data corruption could cause a safety check to fail
	// We keep this disabled regardless because it is generally better to output a T-pose than to have a
	// potential crash. Corruption can happen and it would be unfortunate if a demo or playtest failed
	// as a result of a crash that we can otherwise recover from.
	//static constexpr bool skip_initialize_safety_checks() { return true; }
#endif
};

struct UEDebugDecompressionSettings : public acl::debug_transform_decompression_settings
{
	// Only support our latest version
	static constexpr acl::compressed_tracks_version16 version_supported() { return acl::compressed_tracks_version16::latest; }
};

// Same as debug settings for now since everything is allowed
using UECustomDecompressionSettings = UEDebugDecompressionSettings;

struct UESafeDecompressionSettings final : public UEDefaultDecompressionSettings
{
	static constexpr bool is_rotation_format_supported(acl::rotation_format8 format) { return format == acl::rotation_format8::quatf_full; }
	static constexpr acl::rotation_format8 get_rotation_format(acl::rotation_format8 /*format*/) { return acl::rotation_format8::quatf_full; }
};

struct UEDefaultDatabaseSettings final : public acl::default_database_settings
{
	// Only support our latest version
	static constexpr acl::compressed_tracks_version16 version_supported() { return acl::compressed_tracks_version16::latest; }
};

struct UEDefaultDBDecompressionSettings final : public UEDefaultDecompressionSettings
{
	using database_settings_type = UEDefaultDatabaseSettings;
};

using UEDebugDatabaseSettings = acl::debug_database_settings;

struct UEDebugDBDecompressionSettings final : public UEDebugDecompressionSettings
{
	using database_settings_type = UEDebugDatabaseSettings;
};

/** UE equivalents for some ACL enums */
/** An enum for ACL rotation formats. */
UENUM()
enum ACLRotationFormat : int
{
	ACLRF_Quat_128 UMETA(DisplayName = "Quat Full Bit Rate"),
	ACLRF_QuatDropW_96 UMETA(DisplayName = "Quat Drop W Full Bit Rate"),
	ACLRF_QuatDropW_Variable UMETA(DisplayName = "Quat Drop W Variable Bit Rate"),
};

/** An enum for ACL Vector3 formats. */
UENUM()
enum ACLVectorFormat : int
{
	ACLVF_Vector3_96 UMETA(DisplayName = "Vector3 Full Bit Rate"),
	ACLVF_Vector3_Variable UMETA(DisplayName = "Vector3 Variable Bit Rate"),
};

/** An enum for ACL compression levels. */
UENUM()
enum ACLCompressionLevel : int
{
	ACLCL_Lowest UMETA(DisplayName = "Lowest"),
	ACLCL_Low UMETA(DisplayName = "Low"),
	ACLCL_Medium UMETA(DisplayName = "Medium"),
	ACLCL_High UMETA(DisplayName = "High"),
	ACLCL_Highest UMETA(DisplayName = "Highest"),
	ACLCL_Automatic UMETA(DisplayName = "Automatic"),
};

/** Editor only utilities */
#if WITH_EDITORONLY_DATA

/**
* An enum to control how to handle UE phantom tracks.
* A phantom tracks are present in an animation sequence but aren't mapped to skeleton bones.
* As such, they cannot be queried by the engine except manually through DecompressBone.
* It should generally be safe to Strip them but we default to Ignore.
* Re-importing the animation sequence should clean up any such stale data.
*/
UENUM()
enum class ACLPhantomTrackMode : uint8
{
	// Ignore phantom tracks and compress them anyway (same as UE codecs).
	Ignore,

	// Strip the phantom track to save memory by collapsing them to the identity transform while maintaining the track ordering.
	Strip,

	// We ignore the phantom tracks and output a warning to the log.
	Warn,
};

#endif

#if WITH_EDITOR
struct FCompressibleAnimData;
struct FPerPlatformFloat;
class UAnimSequence;
class ITargetPlatform;

ACLPLUGIN_API acl::rotation_format8 GetRotationFormat(ACLRotationFormat Format);
ACLPLUGIN_API acl::vector_format8 GetVectorFormat(ACLVectorFormat Format);
ACLPLUGIN_API acl::compression_level8 GetCompressionLevel(ACLCompressionLevel Level);

ACLPLUGIN_API acl::track_array_qvvf BuildACLTransformTrackArray(ACLAllocator& AllocatorImpl, const FCompressibleAnimData& CompressibleAnimData,
	float DefaultVirtualVertexDistance, float SafeVirtualVertexDistance,
	bool bBuildAdditiveBase, ACLPhantomTrackMode PhantomTrackMode);

/** Compatibility utilities */
ACLPLUGIN_API uint32 GetNumSamples(const FCompressibleAnimData& CompressibleAnimData);
ACLPLUGIN_API float GetSequenceLength(const UAnimSequence& AnimSeq);

namespace ACL
{
	namespace Private
	{
		ACLPLUGIN_API float GetPerPlatformFloat(const FPerPlatformFloat& PerPlatformFloat, const ITargetPlatform* TargetPlatform);
	}
}
#endif // WITH_EDITOR
