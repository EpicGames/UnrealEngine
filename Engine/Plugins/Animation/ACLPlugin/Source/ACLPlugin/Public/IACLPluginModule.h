// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/** The main ACL plugin module interface. */
class IACLPlugin : public IModuleInterface
{
public:
	static inline IACLPlugin& Get()
	{
		return FModuleManager::LoadModuleChecked<IACLPlugin>("ACLPlugin");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("ACLPlugin");
	}
};
