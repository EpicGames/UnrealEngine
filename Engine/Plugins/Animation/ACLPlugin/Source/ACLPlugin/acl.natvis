<?xml version="1.0" encoding="utf-8"?>

<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Type Name="acl::string">
    <DisplayString>{m_c_str,s8}</DisplayString>
  </Type>

  <Type Name="acl::error_result">
    <DisplayString Condition="m_error == nullptr">no error</DisplayString>
    <DisplayString Condition="m_error != nullptr">{m_error,s8}</DisplayString>
  </Type>

  <Type Name="acl::bitset_description">
    <DisplayString>size={m_size}, num_bits={m_size * 32}</DisplayString>
  </Type>

  <Type Name="acl::track">
    <DisplayString Condition="m_name.m_c_str != nullptr">num_samples={m_num_samples}, type={m_type}, name={m_name}</DisplayString>
    <DisplayString>num_samples={m_num_samples}, type={m_type}</DisplayString>
    <Expand>
      <Item Name="name" Condition="m_name.m_c_str != nullptr">m_name</Item>
      <Item Name="num_samples">m_num_samples</Item>
      <Item Name="sample_rate">m_sample_rate</Item>
      <Item Name="stride">m_stride</Item>
      <Item Name="type">m_type</Item>
      <Item Name="precision" Condition="m_category == acl::track_category8::scalarf">m_desc.scalar.precision</Item>
      <Item Name="precision" Condition="m_category == acl::track_category8::transformf">m_desc.transform.precision</Item>
      <Item Name="shell_distance" Condition="m_category == acl::track_category8::transformf">m_desc.transform.shell_distance</Item>
      <Item Name="output_index" Condition="m_category == acl::track_category8::scalarf">m_desc.scalar.output_index</Item>
      <Item Name="output_index" Condition="m_category == acl::track_category8::transformf">m_desc.transform.output_index</Item>
      <Item Name="parent_index" Condition="m_category == acl::track_category8::transformf">m_desc.transform.parent_index</Item>
      <Item Name="default_value" Condition="m_category == acl::track_category8::transformf">m_desc.transform.default_value</Item>
      <Item Name="is_ref">m_allocator == nullptr</Item>
      <ArrayItems>
        <Size>m_num_samples</Size>
        <!-- TODO: Support stride -->
        <ValuePointer Condition="m_type == acl::track_type8::float1f">(float*)m_data</ValuePointer>
        <ValuePointer Condition="m_type == acl::track_type8::float2f">(rtm::float2f*)m_data</ValuePointer>
        <ValuePointer Condition="m_type == acl::track_type8::float3f">(rtm::float3f*)m_data</ValuePointer>
        <ValuePointer Condition="m_type == acl::track_type8::float4f">(rtm::float4f*)m_data</ValuePointer>
        <ValuePointer Condition="m_type == acl::track_type8::vector4f">(rtm::vector4f*)m_data</ValuePointer>
        <ValuePointer Condition="m_type == acl::track_type8::qvvf">(rtm::qvvf*)m_data</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="acl::track_array">
    <DisplayString Condition="m_name.m_c_str != nullptr">num_tracks={m_num_tracks}, name={m_name}</DisplayString>
    <DisplayString>num_tracks={m_num_tracks}</DisplayString>
    <Expand>
      <Item Name="name" Condition="m_name.m_c_str != nullptr">m_name</Item>
      <Item Name="num_tracks">m_num_tracks</Item>
      <ArrayItems>
        <Size>m_num_tracks</Size>
        <ValuePointer>m_tracks</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="acl::ptr_offset&lt;*,*&gt;">
    <DisplayString Condition="m_value == k_invalid_value">Invalid!</DisplayString>
    <DisplayString>{m_value}</DisplayString>
  </Type>

  <Type Name="acl::compressed_tracks">
    <DisplayString>size={m_buffer_header.size} hash={m_buffer_header.hash}</DisplayString>
    <Expand>
      <!-- Common -->
      <Item Name="num_tracks">((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->num_tracks</Item>
      <Item Name="num_samples">((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->num_samples</Item>
      <Item Name="sample_rate">((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->sample_rate</Item>
      <Item Name="track_type">((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type</Item>
      <Item Name="version">((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->version</Item>
      <!-- Transform tracks -->
      <Item Name="has_scale" Condition="((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type == acl::track_type8::qvvf">(((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->misc_packed &amp; 1) != 0</Item>
      <Item Name="default_scale" Condition="((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type == acl::track_type8::qvvf">(float)((((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->misc_packed &gt;&gt; 1) &amp; 1)</Item>
      <Item Name="rotation_format" Condition="((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type == acl::track_type8::qvvf">(acl::rotation_format8)((((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->misc_packed &gt;&gt; 4) &amp; 15)</Item>
      <Item Name="translation_format" Condition="((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type == acl::track_type8::qvvf">(acl::vector_format8)((((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->misc_packed &gt;&gt; 3) &amp; 1)</Item>
      <Item Name="scale_format" Condition="((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type == acl::track_type8::qvvf">(acl::vector_format8)((((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->misc_packed &gt;&gt; 2) &amp; 1)</Item>
      <Item Name="has_database" Condition="((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type == acl::track_type8::qvvf">(((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->misc_packed &amp; (1 &lt;&lt; 8)) != 0</Item>
      <Item Name="has_trivial_default_values" Condition="((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type == acl::track_type8::qvvf">(((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->misc_packed &amp; (1 &lt;&lt; 9)) != 0</Item>
      <Item Name="has_stripped_keyframes" Condition="((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type == acl::track_type8::qvvf">(((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->misc_packed &amp; (1 &lt;&lt; 10)) != 0</Item>
      <Item Name="has_metadata" Condition="((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type == acl::track_type8::qvvf">(((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->misc_packed &gt;&gt; 31) != 0</Item>
      <Item Name="header" Condition="((const acl::acl_impl::tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->track_type == acl::track_type8::qvvf">(const acl::acl_impl::transform_tracks_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header) + sizeof(acl::acl_impl::tracks_header))</Item>
    </Expand>
  </Type>

  <Type Name="acl::compressed_database">
    <DisplayString>size={m_buffer_header.size} hash={m_buffer_header.hash}</DisplayString>
    <Expand>
      <Item Name="bulk_data_size">((const acl::acl_impl::database_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->bulk_data_size</Item>
      <Item Name="bulk_data_hash">((const acl::acl_impl::database_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->bulk_data_hash</Item>
      <Item Name="num_chunks">((const acl::acl_impl::database_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->num_chunks</Item>
      <Item Name="num_clips">((const acl::acl_impl::database_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->num_clips</Item>
      <Item Name="num_segments">((const acl::acl_impl::database_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->num_segments</Item>
      <Item Name="version">((const acl::acl_impl::database_header*)(((const uint8_t*)this) + sizeof(acl::acl_impl::raw_buffer_header)))->version</Item>
    </Expand>
  </Type>
  
  <!-- Internal implementation detail types follow -->

  <Type Name="acl::acl_impl::scalarf_range">
    <DisplayString>min={m_min}, extent={m_extent}</DisplayString>
  </Type>

  <Type Name="acl::acl_impl::track_range">
    <DisplayString Condition="category == acl::track_category8::scalarf">{range.scalarf}</DisplayString>
    <Expand>
      <Item Name="category">category</Item>
      <Item Name="min" Condition="category == acl::track_category8::scalarf">range.scalarf.m_min</Item>
      <Item Name="extent" Condition="category == acl::track_category8::scalarf">range.scalarf.m_extent</Item>
    </Expand>
  </Type>

  <Type Name="acl::acl_impl::database_tier_mapping">
    <DisplayString>{tier}</DisplayString>
    <Expand>
      <Item Name="num_frames">num_frames</Item>
      <ArrayItems>
        <Size>num_frames</Size>
        <ValuePointer>frames</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="acl::acl_impl::clip_contributing_error">
    <DisplayString>num_segments: {num_segments}, assigned: {num_assigned} / {num_frames}</DisplayString>
    <Expand>
      <Item Name="num_segments">num_segments</Item>
      <Item Name="num_frames">num_frames</Item>
      <Item Name="num_assigned">num_assigned</Item>
      <ArrayItems>
        <Size>num_segments</Size>
        <ValuePointer>segments</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="acl::acl_impl::segment_contriguting_error">
    <DisplayString>assigned: {num_assigned} / {num_movable}</DisplayString>
    <Expand>
      <Item Name="num_frames">num_frames</Item>
      <Item Name="num_movable">num_movable</Item>
      <Item Name="num_assigned">num_assigned</Item>
      <ArrayItems>
        <Size>num_frames</Size>
        <ValuePointer>errors</ValuePointer>
      </ArrayItems>
    </Expand>
  </Type>

  <Type Name="acl::acl_impl::packed_sub_track_types">
    <Expand>
      <IndexListItems>
        <Size>16</Size>
        <ValueNode Condition="((types >> ((15 - $i) * 2)) &amp; 0x3) == 0">"default"</ValueNode>
        <ValueNode Condition="((types >> ((15 - $i) * 2)) &amp; 0x3) == 1">"constant"</ValueNode>
        <ValueNode Condition="((types >> ((15 - $i) * 2)) &amp; 0x3) == 2">"animated"</ValueNode>
      </IndexListItems>
    </Expand>
  </Type>

  <Type Name="acl::acl_impl::track_stream">
    <DisplayString>num_samples={m_num_samples}, bit_rate={m_bit_rate}, type={m_type}</DisplayString>
    <Expand>
      <Item Name="num_samples">m_num_samples</Item>
      <Item Name="sample_rate">m_sample_rate</Item>
      <Item Name="type">m_type</Item>
      <Item Name="bit_rate">m_bit_rate</Item>
      <Item Name="format">m_format</Item>
      <ArrayItems>
        <Size>m_num_samples</Size>
        <!-- TODO: Support stride -->
        <ValuePointer Condition="m_type == acl::animation_track_type8::rotation">(rtm::float4f*)m_samples</ValuePointer>
        <ValuePointer Condition="m_type == acl::animation_track_type8::translation">(rtm::float4f*)m_samples</ValuePointer>
        <ValuePointer Condition="m_type == acl::animation_track_type8::scale">(rtm::float4f*)m_samples</ValuePointer>
      </ArrayItems>
    </Expand>

  </Type>

</AutoVisualizer>
