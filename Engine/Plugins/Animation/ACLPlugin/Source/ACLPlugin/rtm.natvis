<?xml version="1.0" encoding="utf-8"?>

<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

	<Type Name="rtm::vector4f" Priority="MediumLow">
		<DisplayString>({x}, {y}, {z}, {w})</DisplayString>
	</Type>

	<Type Name="rtm::quatf" Priority="MediumLow">
		<DisplayString>({x}, {y}, {z}, {w})</DisplayString>
	</Type>

	<Type Name="rtm::vector4d" Priority="MediumLow">
		<DisplayString>({x}, {y}, {z}, {w})</DisplayString>
	</Type>

	<Type Name="rtm::vector4d">
		<DisplayString>({xy.m128d_f64[0]}, {xy.m128d_f64[1]}, {zw.m128d_f64[0]}, {zw.m128d_f64[1]})</DisplayString>
	</Type>

	<Type Name="rtm::quatd" Priority="MediumLow">
		<DisplayString>({x}, {y}, {z}, {w})</DisplayString>
	</Type>

	<Type Name="rtm::quatd">
		<DisplayString>({xy.m128d_f64[0]}, {xy.m128d_f64[1]}, {zw.m128d_f64[0]}, {zw.m128d_f64[1]})</DisplayString>
	</Type>

	<Type Name="rtm::rtm_impl::angle_constant">
		<DisplayString>Degrees: {dbl * (180.0 / 3.14159265358979323846)}</DisplayString>
	</Type>

	<Type Name="rtm::float2f">
		<DisplayString>({x}, {y})</DisplayString>
	</Type>

	<Type Name="rtm::float3f">
		<DisplayString>({x}, {y}, {z})</DisplayString>
	</Type>

	<Type Name="rtm::float4f">
		<DisplayString>({x}, {y}, {z}, {w})</DisplayString>
	</Type>

	<Type Name="rtm::float2d">
		<DisplayString>({x}, {y})</DisplayString>
	</Type>

	<Type Name="rtm::float3d">
		<DisplayString>({x}, {y}, {z})</DisplayString>
	</Type>

	<Type Name="rtm::float4d">
		<DisplayString>({x}, {y}, {z}, {w})</DisplayString>
	</Type>

</AutoVisualizer>
