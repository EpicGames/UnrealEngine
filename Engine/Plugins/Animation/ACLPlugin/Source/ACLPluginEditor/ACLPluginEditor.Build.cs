// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2020 Nicholas Frechette. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class ACLPluginEditor : ModuleRules
	{
		public ACLPluginEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			string ACLSDKDir = Path.GetFullPath(Path.Combine(ModuleDirectory, "../ThirdParty"));

			PublicIncludePaths.Add(Path.Combine(ModuleDirectory, "Public"));
			PublicIncludePaths.Add(Path.Combine(ACLSDKDir, "acl/external/sjson-cpp/includes"));

			PublicDependencyModuleNames.Add("ACLPlugin");
			PublicDependencyModuleNames.Add("AnimationDataController");
			PublicDependencyModuleNames.Add("Core");
			PublicDependencyModuleNames.Add("CoreUObject");
			PublicDependencyModuleNames.Add("Engine");

			PrivateDependencyModuleNames.Add("EditorStyle");
			PrivateDependencyModuleNames.Add("Slate");
			PrivateDependencyModuleNames.Add("SlateCore");
			PrivateDependencyModuleNames.Add("UnrealEd");

			PrivateDefinitions.Add("ACL_USE_SJSON");
		}
	}
}
