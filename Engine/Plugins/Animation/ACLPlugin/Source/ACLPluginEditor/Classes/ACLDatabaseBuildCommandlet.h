// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2021 Nicholas Frechette. All Rights Reserved.

#include "Commandlets/Commandlet.h"
#include "ACLDatabaseBuildCommandlet.generated.h"

/*
 * This commandlet is used to update instances of UAnimationCompressionLibraryDatabase to ensure their mapping is up-to-date.
 */
UCLASS()
class UACLDatabaseBuildCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	virtual int32 Main(const FString& Params) override;
};
