// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "Commandlets/Commandlet.h"
#include "ACLStatsDumpCommandlet.generated.h"

/*
 * This commandlet is used to extract and dump animation compression statistics.
 *
 * See cpp implementation for example usage and supported arguments.
 */
UCLASS()
class UACLStatsDumpCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:
	virtual int32 Main(const FString& Params) override;

	FString ACLRawDir;
	FString OutputDir;

	bool PerformExhaustiveDump;
	bool PerformCompression;
	bool PerformClipExtraction;
	bool TryAutomaticCompression;
	bool TryACLCompression;
	bool TryKeyReductionRetarget;
	bool TryKeyReduction;
	bool ResumeTask;
	bool SkipAdditiveClips;

	class UAnimBoneCompressionSettings* AutoCompressionSettings;
	class UAnimBoneCompressionSettings* ACLCompressionSettings;
	class UAnimBoneCompressionSettings* KeyReductionCompressionSettings;
	class UAnimBoneCompressionCodec_ACL* ACLCodec;
	class UAnimCompress_RemoveLinearKeys* KeyReductionCodec;
};
