// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2021 Nicholas Frechette. All Rights Reserved.

#include "ACLDatabaseBuildCommandlet.h"

#include "AnimationCompressionLibraryDatabase.h"

#include "AnimationCompression.h"
#include "AnimationUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "FileHelpers.h"
#include "ISourceControlModule.h"
#include "SourceControlOperations.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ACLDatabaseBuildCommandlet)


//////////////////////////////////////////////////////////////////////////
// Commandlet example inspired by: https://github.com/ue4plugins/CommandletPlugin
// To run the commandlet, add to the commandline: "$(SolutionDir)$(ProjectName).uproject" -run=/Script/ACLPluginEditor.ACLDatabaseBuild

//////////////////////////////////////////////////////////////////////////

UACLDatabaseBuildCommandlet::UACLDatabaseBuildCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	IsClient = false;
	IsServer = false;
	IsEditor = true;
	LogToConsole = true;
	ShowErrorCount = true;
}

int32 UACLDatabaseBuildCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamsMap;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamsMap);

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	TArray<FAssetData> DatabaseAssets;
	{
		UE_LOGF(LogAnimationCompression, Log, "Retrieving all ACL databases from current project ...");

		FARFilter DatabaseFilter;
		DatabaseFilter.ClassPaths.Add(UAnimationCompressionLibraryDatabase::StaticClass()->GetClassPathName());
		AssetRegistryModule.Get().GetAssets(DatabaseFilter, DatabaseAssets);
	}

	if (DatabaseAssets.Num() == 0)
	{
		UE_LOGF(LogAnimationCompression, Log, "Failed to find any ACL databases, done");
		return 0;
	}

	TArray<FAssetData> AnimSequenceAssets;
	{
		UE_LOGF(LogAnimationCompression, Log, "Retrieving all animation sequences from current project ...");

		FARFilter AnimSequenceFilter;
		AnimSequenceFilter.ClassPaths.Add(UAnimSequence::StaticClass()->GetClassPathName());
		AssetRegistryModule.Get().GetAssets(AnimSequenceFilter, AnimSequenceAssets);
	}

	if (AnimSequenceAssets.Num() == 0)
	{
		UE_LOGF(LogAnimationCompression, Log, "Failed to find any animation sequences, done");
		return 0;
	}

	{
		UE_LOGF(LogAnimationCompression, Log, "Loading %u animation sequences ...", AnimSequenceAssets.Num());
		for (const FAssetData& Asset : AnimSequenceAssets)
		{
			UAnimSequence* AnimSeq = Cast<UAnimSequence>(Asset.GetAsset());
			if (AnimSeq == nullptr)
			{
				UE_LOGF(LogAnimationCompression, Log, "Failed to load animation sequence: %ls", *Asset.PackagePath.ToString());
				continue;
			}

			// Make sure all our required dependencies are loaded
			FAnimationUtils::EnsureAnimSequenceLoaded(*AnimSeq);
		}
	}

	{
		UE_LOGF(LogAnimationCompression, Log, "Loading %u ACL databases ...", DatabaseAssets.Num());
		for (const FAssetData& Asset : DatabaseAssets)
		{
			UAnimationCompressionLibraryDatabase* Database = Cast<UAnimationCompressionLibraryDatabase>(Asset.GetAsset());
			if (Database == nullptr)
			{
				UE_LOGF(LogAnimationCompression, Log, "Failed to load ACL database: %ls", *Asset.PackagePath.ToString());
				continue;
			}
		}
	}

	TArray<UPackage*> DirtyDatabasePackages;
	{
		for (const FAssetData& Asset : DatabaseAssets)
		{
			UE_LOGF(LogAnimationCompression, Log, "Building mapping for ACL database: %ls ...", *Asset.PackagePath.ToString());

			UAnimationCompressionLibraryDatabase* Database = Cast<UAnimationCompressionLibraryDatabase>(Asset.GetAsset());
			if (Database == nullptr)
			{
				continue;
			}

			const bool bIsDirty = Database->UpdateReferencingAnimSequenceList();
			if (bIsDirty)
			{
				UE_LOGF(LogAnimationCompression, Log, "    Mapping updated!");
				DirtyDatabasePackages.Add(Asset.GetPackage());
			}
		}
	}

	bool bFailedToSave = false;
	{
		if (ISourceControlModule::Get().IsEnabled())
		{
			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

			TArray<UPackage*> PackagesToSave;
			for (UPackage* Package : DirtyDatabasePackages)
			{
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Package, EStateCacheUsage::Use);
				if (SourceControlState->IsCheckedOutOther())
				{
					UE_LOGF(LogAnimationCompression, Warning, "Package %ls is already checked out by someone, will not check out", *SourceControlState->GetFilename());
				}
				else if (!SourceControlState->IsCurrent())
				{
					UE_LOGF(LogAnimationCompression, Warning, "Package %ls is not at head, will not check out", *SourceControlState->GetFilename());
				}
				else if (SourceControlState->CanCheckout())
				{
					const ECommandResult::Type StatusResult = SourceControlProvider.Execute(ISourceControlOperation::Create<FCheckOut>(), Package);
					if (StatusResult != ECommandResult::Succeeded)
					{
						UE_LOGF(LogAnimationCompression, Log, "Package %ls failed to check out", *SourceControlState->GetFilename());
						bFailedToSave = true;
					}
					else
					{
						PackagesToSave.Add(Package);
					}
				}
				else if (!SourceControlState->IsSourceControlled() || SourceControlState->CanEdit())
				{
					PackagesToSave.Add(Package);
				}
			}

			UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, true);
			ISourceControlModule::Get().QueueStatusUpdate(PackagesToSave);
		}
		else
		{
			// No source control, just try to save what we have
			UEditorLoadingAndSavingUtils::SavePackages(DirtyDatabasePackages, true);
		}
	}

	return bFailedToSave ? 1 : 0;
}

