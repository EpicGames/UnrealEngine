// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "CoreMinimal.h"
#include "IACLPluginEditorModule.h"
#include "Modules/ModuleManager.h"

#include "AssetToolsModule.h"

#include "AssetTypeActions_AnimationCompressionLibraryDatabase.h"

class FACLPluginEditor final : public IACLPluginEditor
{
private:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void OnPostEngineInit();
	void RegisterAssetTypeAction(IAssetTools& AssetTools, TSharedRef<IAssetTypeActions> Action);

	TArray<TSharedPtr<IAssetTypeActions>> RegisteredAssetTypeActions;
};

IMPLEMENT_MODULE(FACLPluginEditor, ACLPluginEditor)

//////////////////////////////////////////////////////////////////////////

void FACLPluginEditor::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FACLPluginEditor::OnPostEngineInit);
}

void FACLPluginEditor::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	// Unregister our asset types
	if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
	{
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
		for (int32 Index = 0; Index < RegisteredAssetTypeActions.Num(); ++Index)
		{
			AssetTools.UnregisterAssetTypeActions(RegisteredAssetTypeActions[Index].ToSharedRef());
		}
	}
	RegisteredAssetTypeActions.Empty();
}

void FACLPluginEditor::OnPostEngineInit()
{
	// Register our asset types
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

	RegisterAssetTypeAction(AssetTools, MakeShareable(new FAssetTypeActions_AnimationCompressionLibraryDatabase));
}

void FACLPluginEditor::RegisterAssetTypeAction(IAssetTools& AssetTools, TSharedRef<IAssetTypeActions> Action)
{
	AssetTools.RegisterAssetTypeActions(Action);
	RegisteredAssetTypeActions.Add(Action);
}
