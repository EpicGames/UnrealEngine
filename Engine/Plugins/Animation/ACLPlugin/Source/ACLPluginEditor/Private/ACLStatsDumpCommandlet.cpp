// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2018 Nicholas Frechette. All Rights Reserved.

#include "ACLStatsDumpCommandlet.h"

#include "HAL/FileManagerGeneric.h"
#include "HAL/PlatformTime.h"
#include "HAL/UnrealMemory.h"
#include "Runtime/CoreUObject/Public/UObject/UObjectIterator.h"
#include "Runtime/Engine/Classes/Animation/AnimBoneCompressionSettings.h"
#include "Runtime/Engine/Classes/Animation/AnimCompress.h"
#include "Runtime/Engine/Classes/Animation/AnimCompress_RemoveLinearKeys.h"
#include "Runtime/Engine/Classes/Animation/Skeleton.h"
#include "Runtime/Engine/Public/AnimationCompression.h"
#include "Runtime/Launch/Resources/Version.h"
#include "Editor/UnrealEd/Public/PackageHelperFunctions.h"
#include "Misc/CoreMisc.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"

#include "AnimDataController.h"

#include "AnimBoneCompressionCodec_ACL.h"
#include "ACLImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ACLStatsDumpCommandlet)

THIRD_PARTY_INCLUDES_START
#include <sjson/parser.h>
#include <sjson/writer.h>

#include <acl/compression/impl/track_list_context.h>	// For create_output_track_mapping(..)
#include <acl/compression/convert.h>
#include <acl/compression/track_array.h>
#include <acl/compression/transform_error_metrics.h>
#include <acl/compression/track_error.h>
#include <acl/decompression/decompress.h>
#include <acl/io/clip_reader.h>
#include <acl/io/clip_writer.h>

#include <rtm/quatf.h>
#include <rtm/qvvf.h>
#include <rtm/vector4f.h>
THIRD_PARTY_INCLUDES_END

//////////////////////////////////////////////////////////////////////////
// Commandlet example inspired by: https://github.com/ue4plugins/CommandletPlugin
// To run the commandlet, add to the commandline: "$(SolutionDir)$(ProjectName).uproject" -run=/Script/ACLPluginEditor.ACLStatsDump "-input=<path/to/raw/acl/sjson/files/directory>" "-output=<path/to/output/stats/directory>" -compress
//
// Usage:
//		-input=<directory>: If present all *acl.sjson files will be used as the input for the commandlet otherwise the current project is used
//		-output=<directory>: The commandlet output will be written at the given path (stats or dumped clips)
//		-compress: Commandlet will compress the input clips and output stats
//		-extract: Commandlet will extract the input clips into output *acl.sjson clips
//		-error: Enables the exhaustive error dumping
//		-resume: If present, clip extraction or compression will continue where it left off
// 
// Codec specific:
//		-auto: Uses automatic compression
//		-ErrorTolerance=<tolerance>: The error threshold used by automatic compression
// 
//		-acl: Uses ACL compression
// 
//		-keyreduction: Use linear key reduction
//		-keyreductionrt: Use linear key reduction with retargetting (error compensation)
//////////////////////////////////////////////////////////////////////////

class UESJSONStreamWriter final : public sjson::StreamWriter
{
public:
	UESJSONStreamWriter(FArchive* File_)
		: File(File_)
	{}

	virtual void write(const void* Buffer, size_t BufferSize) override
	{
		File->Serialize(const_cast<void*>(Buffer), BufferSize);
	}

private:
	FArchive* File;
};

static const TCHAR* ReadACLClip(FFileManagerGeneric& FileManager, const FString& ACLClipPath, acl::iallocator& Allocator, acl::track_array_qvvf& OutTracks)
{
	FArchive* Reader = FileManager.CreateFileReader(*ACLClipPath);
	const int64 Size = Reader->TotalSize();

	// Allocate directly without a TArray to automatically manage the memory because some
	// clips are larger than 2 GB
	char* RawData = static_cast<char*>(FMemory::Malloc(Size));

	Reader->Serialize(RawData, Size);
	Reader->Close();

	if (ACLClipPath.EndsWith(TEXT(".acl")))
	{
		acl::compressed_tracks* CompressedTracks = reinterpret_cast<acl::compressed_tracks*>(RawData);
		if (Size != CompressedTracks->get_size() || CompressedTracks->is_valid(true).any())
		{
			FMemory::Free(RawData);
			return TEXT("Invalid binary ACL file provided");
		}

		const acl::error_result Result = acl::convert_track_list(Allocator, *CompressedTracks, OutTracks);
		if (Result.any())
		{
			FMemory::Free(RawData);
			return TEXT("Failed to convert input binary track list");
		}
	}
	else
	{
		acl::clip_reader ClipReader(Allocator, RawData, Size);

		if (ClipReader.get_file_type() != acl::sjson_file_type::raw_clip)
		{
			FMemory::Free(RawData);
			return TEXT("SJSON file isn't a raw clip");
		}

		acl::sjson_raw_clip RawClip;
		if (!ClipReader.read_raw_clip(RawClip))
		{
			FMemory::Free(RawData);
			return TEXT("Failed to read ACL raw clip from file");
		}

		OutTracks = MoveTemp(RawClip.track_list);
	}

	FMemory::Free(RawData);
	return nullptr;
}

static FString GetBoneName(const acl::track_qvvf& Track)
{
	// We add a prefix to ensure the name is safe for ControlRig in 5.x
	return FString::Printf(TEXT("ACL_%s"), ANSI_TO_TCHAR(Track.get_name().c_str()));
}

static void ConvertSkeleton(const acl::track_array_qvvf& Tracks, USkeleton* UESkeleton)
{
	// Not terribly clean, we cast away the 'const' to modify the skeleton
	FReferenceSkeleton& RefSkeleton = const_cast<FReferenceSkeleton&>(UESkeleton->GetReferenceSkeleton());
	FReferenceSkeletonModifier SkeletonModifier(RefSkeleton, UESkeleton);

	for (const acl::track_qvvf& Track : Tracks)
	{
		const acl::track_desc_transformf& Desc = Track.get_description();

		const FString BoneName = GetBoneName(Track);

		FMeshBoneInfo UEBone;
		UEBone.Name = FName(*BoneName);
		UEBone.ParentIndex = Desc.parent_index == acl::k_invalid_track_index ? INDEX_NONE : Desc.parent_index;
		UEBone.ExportName = BoneName;

		const FTransform BindPose = ACLTransformToUE(Desc.default_value);

		SkeletonModifier.Add(UEBone, BindPose);
	}

	// When our modifier is destroyed here, it will rebuild the skeleton
}

static void ConvertClip(const acl::track_array_qvvf& Tracks, UAnimSequence* UEClip, USkeleton* UESkeleton)
{
	UEClip->SetSkeleton(UESkeleton);

	const int32 NumSamples = Tracks.get_num_samples_per_track();	// int32 for 5.2 FFrameNumber constructor
	const float SequenceLength = FGenericPlatformMath::Max<float>(Tracks.get_finite_duration(), MINIMUM_ANIMATION_LENGTH);

	const float SampleRate = Tracks.get_sample_rate();

	// This is incorrect because the true sample rate can be fractional but UE doesn't support it
	const uint32 FrameRate = FGenericPlatformMath::RoundToInt(SampleRate);

	IAnimationDataController& UEClipController = UEClip->GetController();
	UEClipController.InitializeModel();
	UEClipController.ResetModel(false);

	UEClipController.OpenBracket(FText::FromString("Generating Animation Data"));

	UEClipController.SetFrameRate(FFrameRate(FrameRate, 1));

	const int32 NumFrames = NumSamples - 1;
	UEClipController.SetNumberOfFrames(FFrameNumber(NumFrames));

	// Ensure our frame rate update propagates first to avoid re-sampling below
	UEClipController.NotifyPopulated();

	if (NumSamples != 0)
	{
		const uint32 NumBones = Tracks.get_num_tracks();
		for (uint32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const acl::track_qvvf& Track = Tracks[BoneIndex];

			FRawAnimSequenceTrack RawTrack;
			RawTrack.PosKeys.Empty();
			RawTrack.RotKeys.Empty();
			RawTrack.ScaleKeys.Empty();

			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				const FQuat4f Rotation = ACLQuatToUE(rtm::quat_normalize(Track[SampleIndex].rotation));
				RawTrack.RotKeys.Add(Rotation);
			}

			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				const FVector3f Translation = ACLVector3ToUE(Track[SampleIndex].translation);
				RawTrack.PosKeys.Add(Translation);
			}

			for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				const FVector3f Scale = ACLVector3ToUE(Track[SampleIndex].scale);
				RawTrack.ScaleKeys.Add(Scale);
			}

			const FName BoneName(*GetBoneName(Track));

			UEClipController.AddBoneCurve(BoneName);
			UEClipController.SetBoneTrackKeys(BoneName, RawTrack.PosKeys, RawTrack.RotKeys, RawTrack.ScaleKeys);
		}
	}

	UEClipController.NotifyPopulated();
	UEClipController.CloseBracket();
}

static int32 GetAnimationTrackIndex(const int32 BoneIndex, const UAnimSequence* AnimSeq)
{
	if (BoneIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = AnimSeq->GetCompressedData();
	const TArray<FTrackToSkeletonMap>& TrackToSkelMap = CompressedAnimSequence.Get().CompressedTrackToSkeletonMapTable;
	for (int32 TrackIndex = 0; TrackIndex < TrackToSkelMap.Num(); ++TrackIndex)
	{
		const FTrackToSkeletonMap& TrackToSkeleton = TrackToSkelMap[TrackIndex];
		if (TrackToSkeleton.BoneTreeIndex == BoneIndex)
		{
			return TrackIndex;
		}
	}

	return INDEX_NONE;
}

static void SampleUEClip(const acl::track_array_qvvf& Tracks, USkeleton* UESkeleton, const UAnimSequence* UEClip, float SampleTime, rtm::qvvf* LossyPoseTransforms)
{
	const FReferenceSkeleton& RefSkeleton = UESkeleton->GetReferenceSkeleton();
	const TArray<FTransform>& RefSkeletonPose = UESkeleton->GetRefLocalPoses();

	const FAnimExtractContext Context(static_cast<double>(SampleTime));
	const uint32 NumBones = Tracks.get_num_tracks();
	for (uint32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const acl::track_qvvf& Track = Tracks[BoneIndex];
		const FName BoneName(*GetBoneName(Track));
		const int32 BoneTreeIndex = RefSkeleton.FindBoneIndex(BoneName);

		FTransform BoneTransform;
		if (BoneTreeIndex != INDEX_NONE)
		{
			BoneTransform = RefSkeletonPose[BoneTreeIndex];

			if (UEClip->GetDataModel()->IsValidBoneTrackName(BoneName))
			{
				UEClip->GetBoneTransform(BoneTransform, FSkeletonPoseBoneIndex(BoneTreeIndex), Context, false);
			}
		}

		const rtm::quatf Rotation = UEQuatToACL(BoneTransform.GetRotation());
		const rtm::vector4f Translation = UEVector3ToACL(BoneTransform.GetTranslation());
		const rtm::vector4f Scale = UEVector3ToACL(BoneTransform.GetScale3D());
		LossyPoseTransforms[BoneIndex] = rtm::qvv_set(Rotation, Translation, Scale);
	}
}

static bool UEClipHasScale(const UAnimSequence* UEClip)
{
	TArray<FName> TrackNames;
	UEClip->GetDataModel()->GetBoneTrackNames(TrackNames);

	bool bHasScaleKeys = false;
	for (const FName& TrackName : TrackNames)
	{
		UEClip->GetDataModel()->IterateBoneKeys(
			TrackName,
			[&bHasScaleKeys](const FVector3f& Position, const FQuat4f& Rotation, const FVector3f& Scale, const FFrameNumber& FrameNumber)
			{
				if (!Scale.IsUnit())
				{
					bHasScaleKeys = true;
					return false;
				}

				return true;
			});

		if (bHasScaleKeys)
		{
			break;
		}
	}

	return bHasScaleKeys;
}

struct SimpleTransformWriter final : public acl::track_writer
{
	//////////////////////////////////////////////////////////////////////////
	// For performance reasons, this writer skips all default sub-tracks.
	// It is the responsibility of the caller to pre-populate them by calling initialize_with_defaults().
	static constexpr acl::default_sub_track_mode get_default_rotation_mode() { return acl::default_sub_track_mode::skipped; }
	static constexpr acl::default_sub_track_mode get_default_translation_mode() { return acl::default_sub_track_mode::skipped; }
	static constexpr acl::default_sub_track_mode get_default_scale_mode() { return acl::default_sub_track_mode::skipped; }

	explicit SimpleTransformWriter(TArray<rtm::qvvf>& Transforms_) : Transforms(Transforms_) {}

	TArray<rtm::qvvf>& Transforms;

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a quaternion rotation value for a specified bone index.
	void RTM_SIMD_CALL write_rotation(uint32_t TrackIndex, rtm::quatf_arg0 Rotation)
	{
		Transforms[TrackIndex].rotation = Rotation;
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a translation value for a specified bone index.
	void RTM_SIMD_CALL write_translation(uint32_t TrackIndex, rtm::vector4f_arg0 Translation)
	{
		Transforms[TrackIndex].translation = Translation;
	}

	//////////////////////////////////////////////////////////////////////////
	// Called by the decoder to write out a scale value for a specified bone index.
	void RTM_SIMD_CALL write_scale(uint32_t TrackIndex, rtm::vector4f_arg0 Scale)
	{
		Transforms[TrackIndex].scale = Scale;
	}
};

static void CalculateClipError(const acl::track_array_qvvf& Tracks, const UAnimSequence* UEClip, USkeleton* UESkeleton, uint32& OutWorstBone, float& OutMaxError, float& OutWorstSampleTime)
{
	// Use the ACL code if we can to calculate the error instead of approximating it with UE.
	UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = UEClip->GetCompressedData();
	UAnimBoneCompressionCodec_ACLBase* ACLCodec = Cast<UAnimBoneCompressionCodec_ACLBase>(CompressedAnimSequence.Get().BoneCompressionCodec);
	if (ACLCodec != nullptr)
	{
		const acl::compressed_tracks* CompressedClipData = acl::make_compressed_tracks(CompressedAnimSequence.Get().CompressedByteStream.GetData());

		const acl::qvvf_transform_error_metric ErrorMetric;

		// Use debug settings since we don't know the specific codec used
		acl::decompression_context<UEDebugDecompressionSettings> Context;
		Context.initialize(*CompressedClipData);
		const acl::track_error TrackError = acl::calculate_compression_error(ACLAllocatorImpl, Tracks, Context, ErrorMetric);

		OutWorstBone = TrackError.index;
		OutMaxError = TrackError.error;
		OutWorstSampleTime = TrackError.sample_time;
		return;
	}

	const uint32 NumBones = Tracks.get_num_tracks();
	const float ClipDuration = Tracks.get_duration();
	const float SampleRate = Tracks.get_sample_rate();
	const uint32 NumSamples = Tracks.get_num_samples_per_track();
	const bool HasScale = UEClipHasScale(UEClip);

	TArray<rtm::qvvf> RawLocalPoseTransforms;
	TArray<rtm::qvvf> RawObjectPoseTransforms;
	TArray<rtm::qvvf> LossyLocalPoseTransforms;
	TArray<rtm::qvvf> LossyObjectPoseTransforms;
	RawLocalPoseTransforms.AddUninitialized(NumBones);
	RawObjectPoseTransforms.AddUninitialized(NumBones);
	LossyLocalPoseTransforms.AddUninitialized(NumBones);
	LossyObjectPoseTransforms.AddUninitialized(NumBones);

	uint32 WorstBone = acl::k_invalid_track_index;
	float MaxError = 0.0f;
	float WorstSampleTime = 0.0f;

	const acl::qvvf_transform_error_metric ErrorMetric;
	SimpleTransformWriter RawWriter(RawLocalPoseTransforms);

	TArray<uint32> ParentTransformIndices;
	TArray<uint32> SelfTransformIndices;
	ParentTransformIndices.AddUninitialized(NumBones);
	SelfTransformIndices.AddUninitialized(NumBones);

	for (uint32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const acl::track_qvvf& Track = Tracks[BoneIndex];
		const acl::track_desc_transformf& Desc = Track.get_description();

		ParentTransformIndices[BoneIndex] = Desc.parent_index;
		SelfTransformIndices[BoneIndex] = BoneIndex;
	}

	acl::itransform_error_metric::local_to_object_space_args local_to_object_space_args_raw;
	local_to_object_space_args_raw.dirty_transform_indices = SelfTransformIndices.GetData();
	local_to_object_space_args_raw.num_dirty_transforms = NumBones;
	local_to_object_space_args_raw.parent_transform_indices = ParentTransformIndices.GetData();
	local_to_object_space_args_raw.local_transforms = RawLocalPoseTransforms.GetData();
	local_to_object_space_args_raw.num_transforms = NumBones;

	acl::itransform_error_metric::local_to_object_space_args local_to_object_space_args_lossy = local_to_object_space_args_raw;
	local_to_object_space_args_lossy.local_transforms = LossyLocalPoseTransforms.GetData();

	for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
	{
		// Sample our streams and calculate the error
		const float SampleTime = rtm::scalar_min(float(SampleIndex) / SampleRate, ClipDuration);

		Tracks.sample_tracks(SampleTime, acl::sample_rounding_policy::none, RawWriter);
		SampleUEClip(Tracks, UESkeleton, UEClip, SampleTime, LossyLocalPoseTransforms.GetData());

		if (HasScale)
		{
			ErrorMetric.local_to_object_space(local_to_object_space_args_raw, RawObjectPoseTransforms.GetData());
			ErrorMetric.local_to_object_space(local_to_object_space_args_lossy, LossyObjectPoseTransforms.GetData());
		}
		else
		{
			ErrorMetric.local_to_object_space_no_scale(local_to_object_space_args_raw, RawObjectPoseTransforms.GetData());
			ErrorMetric.local_to_object_space_no_scale(local_to_object_space_args_lossy, LossyObjectPoseTransforms.GetData());
		}

		for (uint32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
		{
			const acl::track_qvvf& Track = Tracks[BoneIndex];
			const acl::track_desc_transformf& Desc = Track.get_description();

			acl::itransform_error_metric::calculate_error_args calculate_error_args;
			calculate_error_args.transform0 = &RawObjectPoseTransforms[BoneIndex];
			calculate_error_args.transform1 = &LossyObjectPoseTransforms[BoneIndex];
			calculate_error_args.construct_sphere_shell(Desc.shell_distance);

			float Error;
			if (HasScale)
			{
				Error = rtm::scalar_cast(ErrorMetric.calculate_error(calculate_error_args));
			}
			else
			{
				Error = rtm::scalar_cast(ErrorMetric.calculate_error_no_scale(calculate_error_args));
			}

			if (Error > MaxError)
			{
				MaxError = Error;
				WorstBone = BoneIndex;
				WorstSampleTime = SampleTime;
			}
		}
	}

	OutWorstBone = WorstBone;
	OutMaxError = MaxError;
	OutWorstSampleTime = WorstSampleTime;
}

static void DumpClipDetailedError(const acl::track_array_qvvf& Tracks, const UAnimSequence* UEClip, USkeleton* UESkeleton, sjson::ObjectWriter& Writer)
{
	const uint32 NumBones = Tracks.get_num_tracks();
	const float ClipDuration = Tracks.get_duration();
	const float SampleRate = Tracks.get_sample_rate();
	const uint32 NumSamples = Tracks.get_num_samples_per_track();
	const bool HasScale = UEClipHasScale(UEClip);

	TArray<rtm::qvvf> RawLocalPoseTransforms;
	TArray<rtm::qvvf> RawObjectPoseTransforms;
	TArray<rtm::qvvf> LossyLocalPoseTransforms;
	TArray<rtm::qvvf> LossyObjectPoseTransforms;
	RawLocalPoseTransforms.AddUninitialized(NumBones);
	RawObjectPoseTransforms.AddUninitialized(NumBones);
	LossyLocalPoseTransforms.AddUninitialized(NumBones);
	LossyObjectPoseTransforms.AddUninitialized(NumBones);

	const acl::qvvf_transform_error_metric ErrorMetric;

	SimpleTransformWriter RawWriter(RawLocalPoseTransforms);

	TArray<uint32> ParentTransformIndices;
	TArray<uint32> SelfTransformIndices;
	ParentTransformIndices.AddUninitialized(NumBones);
	SelfTransformIndices.AddUninitialized(NumBones);

	for (uint32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
	{
		const acl::track_qvvf& Track = Tracks[BoneIndex];
		const acl::track_desc_transformf& Desc = Track.get_description();

		ParentTransformIndices[BoneIndex] = Desc.parent_index;
		SelfTransformIndices[BoneIndex] = BoneIndex;
	}

	acl::itransform_error_metric::local_to_object_space_args local_to_object_space_args_raw;
	local_to_object_space_args_raw.dirty_transform_indices = SelfTransformIndices.GetData();
	local_to_object_space_args_raw.num_dirty_transforms = NumBones;
	local_to_object_space_args_raw.parent_transform_indices = ParentTransformIndices.GetData();
	local_to_object_space_args_raw.local_transforms = RawLocalPoseTransforms.GetData();
	local_to_object_space_args_raw.num_transforms = NumBones;

	acl::itransform_error_metric::local_to_object_space_args local_to_object_space_args_lossy = local_to_object_space_args_raw;
	local_to_object_space_args_lossy.local_transforms = LossyLocalPoseTransforms.GetData();

	UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = UEClip->GetCompressedData();
	// Use the ACL code if we can to calculate the error instead of approximating it with UE.
	const UAnimBoneCompressionCodec_ACLBase* ACLCodec = Cast<const UAnimBoneCompressionCodec_ACLBase>(CompressedAnimSequence.Get().BoneCompressionCodec);
	if (ACLCodec != nullptr)
	{
		uint32 NumOutputBones = 0;
		uint32* OutputBoneMapping = acl::acl_impl::create_output_track_mapping(ACLAllocatorImpl, Tracks, NumOutputBones);

		TArray<rtm::qvvf> LossyRemappedLocalPoseTransforms;
		LossyRemappedLocalPoseTransforms.AddUninitialized(NumBones);

		local_to_object_space_args_lossy.local_transforms = LossyRemappedLocalPoseTransforms.GetData();

		const acl::compressed_tracks* CompressedClipData = acl::make_compressed_tracks(CompressedAnimSequence.Get().CompressedByteStream.GetData());

		acl::decompression_context<acl::debug_transform_decompression_settings> Context;
		Context.initialize(*CompressedClipData);

		SimpleTransformWriter PoseWriter(LossyLocalPoseTransforms);

		// Initialize the output pose with our default values (possibly bind pose) since default sub-tracks will be skipped
		// to handle stripping
		for (const acl::track_qvvf& track : Tracks)
		{
			const acl::track_desc_transformf& desc = track.get_description();
			if (desc.output_index == acl::k_invalid_track_index)
				continue;	// Stripped, skip it

			LossyLocalPoseTransforms[desc.output_index] = desc.default_value;
		}

		Writer["error_per_frame_and_bone"] = [&](sjson::ArrayWriter& Writer)	//-V1047
		{
			for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
			{
				// Sample our streams and calculate the error
				const float SampleTime = rtm::scalar_min(float(SampleIndex) / SampleRate, ClipDuration);

				Tracks.sample_tracks(SampleTime, acl::sample_rounding_policy::none, RawWriter);

				Context.seek(SampleTime, acl::sample_rounding_policy::none);
				Context.decompress_tracks(PoseWriter);

				// Perform remapping by copying the raw pose first and we overwrite with the decompressed pose if
				// the data is available
				LossyRemappedLocalPoseTransforms = RawLocalPoseTransforms;
				for (uint32 OutputIndex = 0; OutputIndex < NumOutputBones; ++OutputIndex)
				{
					const uint32 BoneIndex = OutputBoneMapping[OutputIndex];
					LossyRemappedLocalPoseTransforms[BoneIndex] = LossyLocalPoseTransforms[OutputIndex];
				}

				if (HasScale)
				{
					ErrorMetric.local_to_object_space(local_to_object_space_args_raw, RawObjectPoseTransforms.GetData());
					ErrorMetric.local_to_object_space(local_to_object_space_args_lossy, LossyObjectPoseTransforms.GetData());
				}
				else
				{
					ErrorMetric.local_to_object_space_no_scale(local_to_object_space_args_raw, RawObjectPoseTransforms.GetData());
					ErrorMetric.local_to_object_space_no_scale(local_to_object_space_args_lossy, LossyObjectPoseTransforms.GetData());
				}

				Writer.push_newline();
				Writer.push([&](sjson::ArrayWriter& Writer)
					{
						for (uint32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
						{
							const acl::track_qvvf& Track = Tracks[BoneIndex];
							const acl::track_desc_transformf& Desc = Track.get_description();

							acl::itransform_error_metric::calculate_error_args calculate_error_args;
							calculate_error_args.transform0 = &RawObjectPoseTransforms[BoneIndex];
							calculate_error_args.transform1 = &LossyObjectPoseTransforms[BoneIndex];
							calculate_error_args.construct_sphere_shell(Desc.shell_distance);

							float Error;
							if (HasScale)
								Error = rtm::scalar_cast(ErrorMetric.calculate_error(calculate_error_args));
							else
								Error = rtm::scalar_cast(ErrorMetric.calculate_error_no_scale(calculate_error_args));

							Writer.push(Error);
						}
					});
			}
		};

		acl::deallocate_type_array(ACLAllocatorImpl, OutputBoneMapping, NumOutputBones);
		return;
	}

	Writer["error_per_frame_and_bone"] = [&](sjson::ArrayWriter& Writer)	//-V1047
	{
		for (uint32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
		{
			// Sample our streams and calculate the error
			const float SampleTime = rtm::scalar_min(float(SampleIndex) / SampleRate, ClipDuration);

			Tracks.sample_tracks(SampleTime, acl::sample_rounding_policy::none, RawWriter);
			SampleUEClip(Tracks, UESkeleton, UEClip, SampleTime, LossyLocalPoseTransforms.GetData());

			if (HasScale)
			{
				ErrorMetric.local_to_object_space(local_to_object_space_args_raw, RawObjectPoseTransforms.GetData());
				ErrorMetric.local_to_object_space(local_to_object_space_args_lossy, LossyObjectPoseTransforms.GetData());
			}
			else
			{
				ErrorMetric.local_to_object_space_no_scale(local_to_object_space_args_raw, RawObjectPoseTransforms.GetData());
				ErrorMetric.local_to_object_space_no_scale(local_to_object_space_args_lossy, LossyObjectPoseTransforms.GetData());
			}

			Writer.push_newline();
			Writer.push([&](sjson::ArrayWriter& Writer)
				{
					for (uint32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
					{
						const acl::track_qvvf& Track = Tracks[BoneIndex];
						const acl::track_desc_transformf& Desc = Track.get_description();

						acl::itransform_error_metric::calculate_error_args calculate_error_args;
						calculate_error_args.transform0 = &RawObjectPoseTransforms[BoneIndex];
						calculate_error_args.transform1 = &LossyObjectPoseTransforms[BoneIndex];
						calculate_error_args.construct_sphere_shell(Desc.shell_distance);

						float Error;
						if (HasScale)
							Error = rtm::scalar_cast(ErrorMetric.calculate_error(calculate_error_args));
						else
							Error = rtm::scalar_cast(ErrorMetric.calculate_error_no_scale(calculate_error_args));

						Writer.push(Error);
					}
				});
		}
	};
}

struct FCompressionContext
{
	UAnimBoneCompressionSettings* AutoCompressor;
	UAnimBoneCompressionSettings* ACLCompressor;
	UAnimBoneCompressionSettings* KeyReductionCompressor;

	UAnimSequence* UEClip;
	USkeleton* UESkeleton;

	acl::track_array_qvvf ACLTracks;

	uint32 ACLRawSize;
	int32 UERawSize;
};

static FString GetCodecName(UAnimBoneCompressionCodec* Codec)
{
	if (Codec == nullptr)
	{
		return TEXT("<null>");
	}

	if (Codec->Description.Len() > 0 && Codec->Description != TEXT("None"))
	{
		return Codec->Description;
	}

	return Codec->GetClass()->GetName();
}

static void CompressWithUEAuto(FCompressionContext& Context, bool PerformExhaustiveDump, sjson::Writer& Writer)
{
	// Force recompression and avoid the DDC
	TGuardValue<int32> CompressGuard(Context.UEClip->CompressCommandletVersion, INDEX_NONE);

	const uint64 UEStartTimeCycles = FPlatformTime::Cycles64();

	Context.UEClip->BoneCompressionSettings = Context.AutoCompressor;

	Context.UEClip->CacheDerivedDataForCurrentPlatform();

	const uint64 UEEndTimeCycles = FPlatformTime::Cycles64();

	const uint64 UEElapsedCycles = UEEndTimeCycles - UEStartTimeCycles;
	const double UEElapsedTimeSec = FPlatformTime::ToSeconds64(UEElapsedCycles);

	if (Context.UEClip->IsBoneCompressedDataValid())
	{
		const UAnimSequence* ConstClip = Context.UEClip;
		UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = ConstClip->GetCompressedData();
		const bool bHasClipData = CompressedAnimSequence.Get().CompressedDataStructure != nullptr;

		AnimationErrorStats UEErrorStats;
		uint32 WorstBone = INDEX_NONE;
		float MaxError = 0.0f;
		float WorstSampleTime = 0.0f;

		if (bHasClipData)
		{
			UEErrorStats = CompressedAnimSequence.Get().CompressedDataStructure->BoneCompressionErrorStats;

			CalculateClipError(Context.ACLTracks, Context.UEClip, Context.UESkeleton, WorstBone, MaxError, WorstSampleTime);
		}

		const int32 CompressedSize = Context.UEClip->GetApproxCompressedSize();
		const double UECompressionRatio = double(Context.UERawSize) / double(CompressedSize);
		const double ACLCompressionRatio = double(Context.ACLRawSize) / double(CompressedSize);

		Writer["ue4_auto"] = [&](sjson::ObjectWriter& Writer)	//-V1047
		{
			Writer["algorithm_name"] = TCHAR_TO_ANSI(*Context.UEClip->BoneCompressionSettings->GetClass()->GetName());
			Writer["codec_name"] = TCHAR_TO_ANSI(*GetCodecName(CompressedAnimSequence.Get().BoneCompressionCodec));
			Writer["compressed_size"] = CompressedSize;
			Writer["ue4_compression_ratio"] = UECompressionRatio;
			Writer["acl_compression_ratio"] = ACLCompressionRatio;
			Writer["compression_time"] = UEElapsedTimeSec;
			Writer["ue4_max_error"] = UEErrorStats.MaxError;
			Writer["ue4_avg_error"] = UEErrorStats.AverageError;
			Writer["ue4_worst_bone"] = UEErrorStats.MaxErrorBone;
			Writer["ue4_worst_time"] = UEErrorStats.MaxErrorTime;
			Writer["acl_max_error"] = MaxError;
			Writer["acl_worst_bone"] = WorstBone;
			Writer["acl_worst_time"] = WorstSampleTime;

			if (CompressedAnimSequence.Get().BoneCompressionCodec != nullptr
				&& CompressedAnimSequence.Get().BoneCompressionCodec->IsA<UAnimCompress>()
				&& bHasClipData)
			{
				const FUECompressedAnimData& AnimData = static_cast<FUECompressedAnimData&>(*CompressedAnimSequence.Get().CompressedDataStructure);
				Writer["rotation_format"] = TCHAR_TO_ANSI(*FAnimationUtils::GetAnimationCompressionFormatString(AnimData.RotationCompressionFormat));
				Writer["translation_format"] = TCHAR_TO_ANSI(*FAnimationUtils::GetAnimationCompressionFormatString(AnimData.TranslationCompressionFormat));
				Writer["scale_format"] = TCHAR_TO_ANSI(*FAnimationUtils::GetAnimationCompressionFormatString(AnimData.ScaleCompressionFormat));
			}

			if (PerformExhaustiveDump && bHasClipData)
			{
				DumpClipDetailedError(Context.ACLTracks, Context.UEClip, Context.UESkeleton, Writer);
			}
		};
	}
	else
	{
		Writer["error"] = "failed to compress UE clip";
	}
}

static void CompressWithACL(FCompressionContext& Context, bool PerformExhaustiveDump, sjson::Writer& Writer)
{
	// Force recompression and avoid the DDC
	TGuardValue<int32> CompressGuard(Context.UEClip->CompressCommandletVersion, INDEX_NONE);

	const uint64 ACLStartTimeCycles = FPlatformTime::Cycles64();

	Context.UEClip->BoneCompressionSettings = Context.ACLCompressor;

	Context.UEClip->CacheDerivedDataForCurrentPlatform();

	const uint64 ACLEndTimeCycles = FPlatformTime::Cycles64();

	const uint64 ACLElapsedCycles = ACLEndTimeCycles - ACLStartTimeCycles;
	const double ACLElapsedTimeSec = FPlatformTime::ToSeconds64(ACLElapsedCycles);

	if (Context.UEClip->IsBoneCompressedDataValid())
	{
		const UAnimSequence* ConstClip = Context.UEClip;
		UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = ConstClip->GetCompressedData();
		const bool bHasClipData = CompressedAnimSequence.Get().CompressedDataStructure != nullptr;

		AnimationErrorStats UEErrorStats;
		uint32 WorstBone = INDEX_NONE;
		float MaxError = 0.0f;
		float WorstSampleTime = 0.0f;

		if (bHasClipData)
		{
			UEErrorStats = CompressedAnimSequence.Get().CompressedDataStructure->BoneCompressionErrorStats;

			CalculateClipError(Context.ACLTracks, Context.UEClip, Context.UESkeleton, WorstBone, MaxError, WorstSampleTime);
		}

		const int32 CompressedSize = Context.UEClip->GetApproxCompressedSize();
		const double UECompressionRatio = double(Context.UERawSize) / double(CompressedSize);
		const double ACLCompressionRatio = double(Context.ACLRawSize) / double(CompressedSize);

		Writer["ue4_acl"] = [&](sjson::ObjectWriter& Writer)	//-V1047
		{
			Writer["algorithm_name"] = TCHAR_TO_ANSI(*Context.UEClip->BoneCompressionSettings->GetClass()->GetName());
			Writer["codec_name"] = TCHAR_TO_ANSI(*GetCodecName(CompressedAnimSequence.Get().BoneCompressionCodec));
			Writer["compressed_size"] = CompressedSize;
			Writer["ue4_compression_ratio"] = UECompressionRatio;
			Writer["acl_compression_ratio"] = ACLCompressionRatio;
			Writer["compression_time"] = ACLElapsedTimeSec;
			Writer["ue4_max_error"] = UEErrorStats.MaxError;
			Writer["ue4_avg_error"] = UEErrorStats.AverageError;
			Writer["ue4_worst_bone"] = UEErrorStats.MaxErrorBone;
			Writer["ue4_worst_time"] = UEErrorStats.MaxErrorTime;
			Writer["acl_max_error"] = MaxError;
			Writer["acl_worst_bone"] = WorstBone;
			Writer["acl_worst_time"] = WorstSampleTime;

			if (PerformExhaustiveDump && bHasClipData)
			{
				DumpClipDetailedError(Context.ACLTracks, Context.UEClip, Context.UESkeleton, Writer);
			}
		};
	}
	else
	{
		Writer["error"] = "failed to compress UE clip";
	}
}

static bool IsKeyDropped(int32 NumFrames, const uint8* FrameTable, int32 NumKeys, float FrameRate, float SampleTime)
{
	if (NumFrames > 0xFF)
	{
		const uint16* Frames = (const uint16*)FrameTable;
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			const float FrameTime = Frames[KeyIndex] / FrameRate;
			if (FMath::IsNearlyEqual(FrameTime, SampleTime, 0.001f))
			{
				return false;
			}
		}
		return true;
	}
	else
	{
		const uint8* Frames = (const uint8*)FrameTable;
		for (int32 KeyIndex = 0; KeyIndex < NumKeys; ++KeyIndex)
		{
			const float FrameTime = Frames[KeyIndex] / FrameRate;
			if (FMath::IsNearlyEqual(FrameTime, SampleTime, 0.001f))
			{
				return false;
			}
		}
		return true;
	}
}

static int32 GetCompressedNumberOfKeys(const FUECompressedAnimData& AnimData)
{
	return AnimData.CompressedNumberOfKeys;
}

static void CompressWithUEKeyReduction(FCompressionContext& Context, bool PerformExhaustiveDump, sjson::Writer& Writer)
{
	using AnimDataModelType = IAnimationDataModel;

	// Force recompression and avoid the DDC
	TGuardValue<int32> CompressGuard(Context.UEClip->CompressCommandletVersion, INDEX_NONE);

	const uint64 UEStartTimeCycles = FPlatformTime::Cycles64();

	Context.UEClip->BoneCompressionSettings = Context.KeyReductionCompressor;

	Context.UEClip->CacheDerivedDataForCurrentPlatform();

	const uint64 UEEndTimeCycles = FPlatformTime::Cycles64();

	const uint64 UEElapsedCycles = UEEndTimeCycles - UEStartTimeCycles;
	const double UEElapsedTimeSec = FPlatformTime::ToSeconds64(UEElapsedCycles);

	if (Context.UEClip->IsBoneCompressedDataValid())
	{
		const UAnimSequence* ConstClip = Context.UEClip;
		UAnimSequence::FScopedCompressedAnimSequence CompressedAnimSequence = ConstClip->GetCompressedData();
		const bool bHasClipData = CompressedAnimSequence.Get().CompressedDataStructure != nullptr;

		AnimationErrorStats UEErrorStats;
		uint32 WorstBone = INDEX_NONE;
		float MaxError = 0.0f;
		float WorstSampleTime = 0.0f;

		if (bHasClipData)
		{
			UEErrorStats = CompressedAnimSequence.Get().CompressedDataStructure->BoneCompressionErrorStats;

			CalculateClipError(Context.ACLTracks, Context.UEClip, Context.UESkeleton, WorstBone, MaxError, WorstSampleTime);
		}

		const int32 CompressedSize = Context.UEClip->GetApproxCompressedSize();
		const double UECompressionRatio = double(Context.UERawSize) / double(CompressedSize);
		const double ACLCompressionRatio = double(Context.ACLRawSize) / double(CompressedSize);

		Writer["ue4_keyreduction"] = [&](sjson::ObjectWriter& Writer)	//-V1047
		{
			Writer["algorithm_name"] = TCHAR_TO_ANSI(*Context.UEClip->BoneCompressionSettings->GetClass()->GetName());
			Writer["codec_name"] = TCHAR_TO_ANSI(*GetCodecName(CompressedAnimSequence.Get().BoneCompressionCodec));
			Writer["compressed_size"] = CompressedSize;
			Writer["ue4_compression_ratio"] = UECompressionRatio;
			Writer["acl_compression_ratio"] = ACLCompressionRatio;
			Writer["compression_time"] = UEElapsedTimeSec;
			Writer["ue4_max_error"] = UEErrorStats.MaxError;
			Writer["ue4_avg_error"] = UEErrorStats.AverageError;
			Writer["ue4_worst_bone"] = UEErrorStats.MaxErrorBone;
			Writer["ue4_worst_time"] = UEErrorStats.MaxErrorTime;
			Writer["acl_max_error"] = MaxError;
			Writer["acl_worst_bone"] = WorstBone;
			Writer["acl_worst_time"] = WorstSampleTime;

			if (PerformExhaustiveDump && bHasClipData)
			{
				DumpClipDetailedError(Context.ACLTracks, Context.UEClip, Context.UESkeleton, Writer);
			}

			// Number of animated keys before any key reduction for animated tracks (without constant/default tracks)
			int32 TotalNumAnimatedKeys = 0;

			// Number of animated keys dropped after key reduction for animated tracks (without constant/default tracks)
			int32 TotalNumDroppedAnimatedKeys = 0;

			// Number of animated tracks (not constant/default)
			int32 NumAnimatedTracks = 0;

			Writer["dropped_track_keys"] = [&](sjson::ArrayWriter& Writer)	//-V1047
			{
				if (!bHasClipData)
				{
					return;	// No data, nothing to append
				}

				const FUECompressedAnimData& AnimData = static_cast<FUECompressedAnimData&>(*CompressedAnimSequence.Get().CompressedDataStructure);

				const AnimDataModelType* ClipData = Context.UEClip->GetDataModel();
				const int32 NumTracks = ClipData->GetNumBoneTracks();
				const int32 NumSamples = ClipData->GetNumberOfFrames();

				const int32* TrackOffsets = AnimData.CompressedTrackOffsets.GetData();
				const auto& ScaleOffsets = AnimData.CompressedScaleOffsets;

				const AnimationCompressionFormat RotationFormat = AnimData.RotationCompressionFormat;
				const AnimationCompressionFormat TranslationFormat = AnimData.TranslationCompressionFormat;
				const AnimationCompressionFormat ScaleFormat = AnimData.ScaleCompressionFormat;

				// offset past Min and Range data
				const int32 RotationStreamOffset = (RotationFormat == ACF_IntervalFixed32NoW) ? (sizeof(float) * 6) : 0;
				const int32 TranslationStreamOffset = (TranslationFormat == ACF_IntervalFixed32NoW) ? (sizeof(float) * 6) : 0;
				const int32 ScaleStreamOffset = (ScaleFormat == ACF_IntervalFixed32NoW) ? (sizeof(float) * 6) : 0;

				for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
				{
					const int32* TrackData = TrackOffsets + (TrackIndex * 4);
					const int32 NumTransKeys = TrackData[1];

					// Skip constant/default tracks
					if (NumTransKeys > 1)
					{
						const int32 DroppedTransCount = NumSamples - NumTransKeys;
						const float DroppedRatio = float(DroppedTransCount) / float(NumSamples);
						Writer.push(DroppedRatio);

						TotalNumAnimatedKeys += NumSamples;
						TotalNumDroppedAnimatedKeys += DroppedTransCount;
						NumAnimatedTracks++;
					}

					const int32 NumRotKeys = TrackData[3];

					// Skip constant/default tracks
					if (NumRotKeys > 1)
					{
						const int32 DroppedRotCount = NumSamples - NumRotKeys;
						const float DroppedRatio = float(DroppedRotCount) / float(NumSamples);
						Writer.push(DroppedRatio);

						TotalNumAnimatedKeys += NumSamples;
						TotalNumDroppedAnimatedKeys += DroppedRotCount;
						NumAnimatedTracks++;
					}

					if (ScaleOffsets.IsValid())
					{
						const int32 NumScaleKeys = ScaleOffsets.GetOffsetData(TrackIndex, 1);

						// Skip constant/default tracks
						if (NumScaleKeys > 1)
						{
							const int32 DroppedScaleCount = NumSamples - NumScaleKeys;
							const float DroppedRatio = float(DroppedScaleCount) / float(NumSamples);
							Writer.push(DroppedRatio);

							TotalNumAnimatedKeys += NumSamples;
							TotalNumDroppedAnimatedKeys += DroppedScaleCount;
							NumAnimatedTracks++;
						}
					}
				}
			};

			Writer["total_num_animated_keys"] = TotalNumAnimatedKeys;
			Writer["total_num_dropped_animated_keys"] = TotalNumDroppedAnimatedKeys;

			Writer["dropped_pose_keys"] = [&](sjson::ArrayWriter& Writer)	//-V1047
			{
				if (!bHasClipData)
				{
					return;	// No data, nothing to append
				}

				const FUECompressedAnimData& AnimData = static_cast<FUECompressedAnimData&>(*CompressedAnimSequence.Get().CompressedDataStructure);

				const AnimDataModelType* ClipData = Context.UEClip->GetDataModel();
				const int32 NumTracks = ClipData->GetNumBoneTracks();
				const int32 NumSamples = ClipData->GetNumberOfFrames();

				const float SequenceLength = GetSequenceLength(*Context.UEClip);
				const int32 NumCompressedKeys = GetCompressedNumberOfKeys(AnimData);

				const float FrameRate = (NumSamples - 1) / SequenceLength;

				const uint8* ByteStream = AnimData.CompressedByteStream.GetData();
				const int32* TrackOffsets = AnimData.CompressedTrackOffsets.GetData();
				const auto& ScaleOffsets = AnimData.CompressedScaleOffsets;

				const AnimationCompressionFormat RotationFormat = AnimData.RotationCompressionFormat;
				const AnimationCompressionFormat TranslationFormat = AnimData.TranslationCompressionFormat;
				const AnimationCompressionFormat ScaleFormat = AnimData.ScaleCompressionFormat;

				// offset past Min and Range data
				const int32 RotationStreamOffset = (RotationFormat == ACF_IntervalFixed32NoW) ? (sizeof(float) * 6) : 0;
				const int32 TranslationStreamOffset = (TranslationFormat == ACF_IntervalFixed32NoW) ? (sizeof(float) * 6) : 0;
				const int32 ScaleStreamOffset = (ScaleFormat == ACF_IntervalFixed32NoW) ? (sizeof(float) * 6) : 0;

				for (int32 SampleIndex = 0; SampleIndex < NumSamples; ++SampleIndex)
				{
					const float SampleTime = float(SampleIndex) / FrameRate;

					int32 DroppedRotCount = 0;
					int32 DroppedTransCount = 0;
					int32 DroppedScaleCount = 0;
					for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
					{
						const int32* TrackData = TrackOffsets + (TrackIndex * 4);

						const int32 TransKeysOffset = TrackData[0];
						const int32 NumTransKeys = TrackData[1];
						const uint8* TransStream = ByteStream + TransKeysOffset;

						const uint8* TransFrameTable = TransStream + TranslationStreamOffset + (NumTransKeys * CompressedTranslationStrides[TranslationFormat] * CompressedTranslationNum[TranslationFormat]);
						TransFrameTable = Align(TransFrameTable, 4);

						// Skip constant/default tracks
						if (NumTransKeys > 1 && IsKeyDropped(NumCompressedKeys, TransFrameTable, NumTransKeys, FrameRate, SampleTime))
						{
							DroppedTransCount++;
						}

						const int32 RotKeysOffset = TrackData[2];
						const int32 NumRotKeys = TrackData[3];
						const uint8* RotStream = ByteStream + RotKeysOffset;

						const uint8* RotFrameTable = RotStream + RotationStreamOffset + (NumRotKeys * CompressedRotationStrides[RotationFormat] * CompressedRotationNum[RotationFormat]);
						RotFrameTable = Align(RotFrameTable, 4);

						// Skip constant/default tracks
						if (NumRotKeys > 1 && IsKeyDropped(NumCompressedKeys, RotFrameTable, NumRotKeys, FrameRate, SampleTime))
						{
							DroppedRotCount++;
						}

						if (ScaleOffsets.IsValid())
						{
							const int32 ScaleKeysOffset = ScaleOffsets.GetOffsetData(TrackIndex, 0);
							const int32 NumScaleKeys = ScaleOffsets.GetOffsetData(TrackIndex, 1);
							const uint8* ScaleStream = ByteStream + ScaleKeysOffset;

							const uint8* ScaleFrameTable = ScaleStream + ScaleStreamOffset + (NumScaleKeys * CompressedScaleStrides[ScaleFormat] * CompressedScaleNum[ScaleFormat]);
							ScaleFrameTable = Align(ScaleFrameTable, 4);

							// Skip constant/default tracks
							if (NumScaleKeys > 1 && IsKeyDropped(NumCompressedKeys, ScaleFrameTable, NumScaleKeys, FrameRate, SampleTime))
							{
								DroppedScaleCount++;
							}
						}
					}

					const int32 TotalDroppedCount = DroppedRotCount + DroppedTransCount + DroppedScaleCount;
					const float DropRatio = NumAnimatedTracks != 0 ? (float(TotalDroppedCount) / float(NumAnimatedTracks)) : 1.0f;
					Writer.push(DropRatio);
				}
			};

#if DO_CHECK && 0
			{
				// Double check our count
				const int32 NumSamples = Context.UEClip->GetRawNumberOfFrames();
				const TArray<FRawAnimSequenceTrack>& RawTracks = Context.UEClip->GetRawAnimationData();
				const int32 NumTracks = RawTracks.Num();
				const int32* TrackOffsets = Context.UEClip->CompressedTrackOffsets.GetData();
				const FCompressedOffsetData& ScaleOffsets = Context.UEClip->CompressedScaleOffsets;

				int32 DroppedRotCount = 0;
				int32 DroppedTransCount = 0;
				int32 DroppedScaleCount = 0;
				for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
				{
					const int32* TrackData = TrackOffsets + (TrackIndex * 4);

					const int32 NumTransKeys = TrackData[1];

					// Skip constant/default tracks
					if (NumTransKeys > 1)
					{
						DroppedTransCount += NumSamples - NumTransKeys;
					}

					const int32 NumRotKeys = TrackData[3];

					// Skip constant/default tracks
					if (NumRotKeys > 1)
					{
						DroppedRotCount += NumSamples - NumRotKeys;
					}

					if (ScaleOffsets.IsValid())
					{
						const int32 NumScaleKeys = ScaleOffsets.GetOffsetData(TrackIndex, 1);

						// Skip constant/default tracks
						if (NumScaleKeys > 1)
						{
							DroppedScaleCount += NumSamples - NumScaleKeys;
						}
					}
				}
				check(TotalNumDroppedKeys == (DroppedRotCount + DroppedTransCount + DroppedScaleCount));
			}
#endif
		};
	}
	else
	{
		Writer["error"] = "failed to compress UE clip";
	}
}

static void ClearClip(UAnimSequence* UEClip)
{
	UEClip->ResetAnimation();
}

struct CompressAnimationsFunctor
{
	template<typename ObjectType>
	void DoIt(UCommandlet* Commandlet, UPackage* Package, const TArray<FString>& Tokens, const TArray<FString>& Switches)
	{
		TArray<UAnimSequence*> AnimSequences;
		for (TObjectIterator<UAnimSequence> It; It; ++It)
		{
			UAnimSequence* AnimSeq = *It;
			if (AnimSeq->IsIn(Package))
			{
				AnimSequences.Add(AnimSeq);
			}
		}

		// Skip packages that contain no Animations.
		const int32 NumAnimSequences = AnimSequences.Num();
		if (NumAnimSequences == 0)
		{
			return;
		}

		UACLStatsDumpCommandlet* StatsCommandlet = Cast<UACLStatsDumpCommandlet>(Commandlet);
		FFileManagerGeneric FileManager;

		for (int32 SequenceIndex = 0; SequenceIndex < NumAnimSequences; ++SequenceIndex)
		{
			UAnimSequence* UEClip = AnimSequences[SequenceIndex];

			// Make sure all our required dependencies are loaded
			FAnimationUtils::EnsureAnimSequenceLoaded(*UEClip);

			USkeleton* UESkeleton = UEClip->GetSkeleton();
			if (UESkeleton == nullptr)
			{
				continue;
			}

			FString Filename = UEClip->GetPathName();
			if (StatsCommandlet->PerformCompression)
			{
				Filename = FString::Printf(TEXT("%X_stats.sjson"), GetTypeHash(Filename));
			}
			else if (StatsCommandlet->PerformClipExtraction)
			{
				Filename = FString::Printf(TEXT("%X.acl.sjson"), GetTypeHash(Filename));
			}

			FString UEOutputPath = FPaths::Combine(*StatsCommandlet->OutputDir, *Filename).Replace(TEXT("/"), TEXT("\\"));

			if (StatsCommandlet->ResumeTask && FileManager.FileExists(*UEOutputPath))
			{
				continue;
			}

			const bool bIsAdditive = UEClip->IsValidAdditive();
			if (bIsAdditive && StatsCommandlet->SkipAdditiveClips)
			{
				continue;
			}

			FCompressionContext Context;
			Context.AutoCompressor = StatsCommandlet->AutoCompressionSettings;
			Context.ACLCompressor = StatsCommandlet->ACLCompressionSettings;
			Context.UEClip = UEClip;
			Context.UESkeleton = UESkeleton;

			FCompressibleAnimData CompressibleData(UEClip, false, GetTargetPlatformManagerRef().GetRunningTargetPlatform());

			acl::track_array_qvvf ACLTracks = BuildACLTransformTrackArray(ACLAllocatorImpl, CompressibleData,
				StatsCommandlet->ACLCodec->DefaultVirtualVertexDistance, StatsCommandlet->ACLCodec->SafeVirtualVertexDistance,
				false, ACLPhantomTrackMode::Ignore);

			// TODO: Add support for additive clips
			//acl::track_array_qvvf ACLBaseTracks;
			//if (CompressibleData.bIsValidAdditive)
				//ACLBaseTracks = BuildACLTransformTrackArray(Allocator, CompressibleData, StatsCommandlet->ACLCodec->DefaultVirtualVertexDistance, StatsCommandlet->ACLCodec->SafeVirtualVertexDistance, true);

			Context.ACLTracks = MoveTemp(ACLTracks);
			Context.ACLRawSize = Context.ACLTracks.get_raw_size();
			Context.UERawSize = UEClip->GetApproxRawSize();

			if (StatsCommandlet->PerformCompression)
			{
				UE_LOGF(LogAnimationCompression, Verbose, "Compressing: %ls (%d / %d)", *UEClip->GetPathName(), SequenceIndex, NumAnimSequences);

				FArchive* OutputWriter = FileManager.CreateFileWriter(*UEOutputPath);
				if (OutputWriter == nullptr)
				{
					// Opening the file handle can fail if the file path is too long on Windows. UE does not properly handle long paths
					// and adding the \\?\ prefix manually doesn't work, UE mangles it when it normalizes the path.
					ClearClip(UEClip);
					continue;
				}

				// Make sure any pending async compression that might have started during load or construction is done
				UEClip->WaitOnExistingCompression();

				UESJSONStreamWriter StreamWriter(OutputWriter);
				sjson::Writer Writer(StreamWriter);

				Writer["duration"] = GetSequenceLength(*UEClip);
				Writer["num_samples"] = GetNumSamples(CompressibleData);
				Writer["ue4_raw_size"] = Context.UERawSize;
				Writer["acl_raw_size"] = Context.ACLRawSize;

				if (StatsCommandlet->TryAutomaticCompression)
				{
					CompressWithUEAuto(Context, StatsCommandlet->PerformExhaustiveDump, Writer);
				}

				if (StatsCommandlet->TryACLCompression)
				{
					CompressWithACL(Context, StatsCommandlet->PerformExhaustiveDump, Writer);
				}

				if (StatsCommandlet->TryKeyReduction)
				{
					CompressWithUEKeyReduction(Context, StatsCommandlet->PerformExhaustiveDump, Writer);
				}

				OutputWriter->Close();
			}
			else if (StatsCommandlet->PerformClipExtraction)
			{
				UE_LOGF(LogAnimationCompression, Verbose, "Extracting: %ls (%d / %d)", *UEClip->GetPathName(), SequenceIndex, NumAnimSequences);

				const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->GetRunningTargetPlatform();

				acl::compression_settings Settings;
				StatsCommandlet->ACLCodec->GetCompressionSettings(TargetPlatform, Settings);

				const acl::error_result Error = acl::write_track_list(Context.ACLTracks, Settings, TCHAR_TO_ANSI(*UEOutputPath));
				if (Error.any())
				{
					UE_LOGF(LogAnimationCompression, Warning, "Failed to write ACL clip file: %ls", ANSI_TO_TCHAR(Error.c_str()));
				}
			}

			ClearClip(UEClip);
		}
	}
};

UACLStatsDumpCommandlet::UACLStatsDumpCommandlet(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	IsClient = false;
	IsServer = false;
	IsEditor = true;
	LogToConsole = true;
	ShowErrorCount = true;
}

static void ClearCompressedData(UAnimSequence* UEClip)
{
	UEClip->ClearAllCachedCookedPlatformData();
}

int32 UACLStatsDumpCommandlet::Main(const FString& Params)
{
	TArray<FString> Tokens;
	TArray<FString> Switches;
	TMap<FString, FString> ParamsMap;
	UCommandlet::ParseCommandLine(*Params, Tokens, Switches, ParamsMap);

	if (!ParamsMap.Contains(TEXT("output")))
	{
		UE_LOGF(LogAnimationCompression, Error, "Missing commandlet argument: -output=<path/to/output/directory>");
		return 0;
	}

	OutputDir = ParamsMap[TEXT("output")];

	PerformExhaustiveDump = Switches.Contains(TEXT("error"));
	PerformCompression = Switches.Contains(TEXT("compress"));
	PerformClipExtraction = Switches.Contains(TEXT("extract"));
	TryAutomaticCompression = Switches.Contains(TEXT("auto"));
	TryACLCompression = Switches.Contains(TEXT("acl"));
	TryKeyReductionRetarget = Switches.Contains(TEXT("keyreductionrt"));
	TryKeyReduction = TryKeyReductionRetarget || Switches.Contains(TEXT("keyreduction"));
	ResumeTask = Switches.Contains(TEXT("resume"));
	SkipAdditiveClips = Switches.Contains(TEXT("noadditive")) || true;	// Disabled for now, TODO add support for it
	const bool HasInput = ParamsMap.Contains(TEXT("input"));

	if (PerformClipExtraction)
	{
		// We don't support extracting additive clips
		SkipAdditiveClips = true;
	}

	if (PerformCompression && PerformClipExtraction)
	{
		UE_LOGF(LogAnimationCompression, Error, "Cannot compress and extract clips at the same time");
		return 0;
	}

	if (!PerformCompression && !PerformClipExtraction)
	{
		UE_LOGF(LogAnimationCompression, Error, "Must compress or extract clips");
		return 0;
	}

	if (PerformClipExtraction && ParamsMap.Contains(TEXT("input")))
	{
		UE_LOGF(LogAnimationCompression, Error, "Cannot use an input directory when extracting clips");
		return 0;
	}

	// Make sure to log everything
	LogAnimationCompression.SetVerbosity(ELogVerbosity::All);

	if (TryAutomaticCompression)
	{
		AutoCompressionSettings = FAnimationUtils::GetDefaultAnimationBoneCompressionSettings();
		AutoCompressionSettings->bForceBelowThreshold = true;

		if (ParamsMap.Contains(TEXT("ErrorTolerance")))
		{
			AutoCompressionSettings->ErrorThreshold = FCString::Atof(*ParamsMap[TEXT("ErrorTolerance")]);
		}
	}

	if (TryACLCompression || !HasInput)
	{
		ACLCompressionSettings = NewObject<UAnimBoneCompressionSettings>(this, UAnimBoneCompressionSettings::StaticClass());
		ACLCodec = NewObject<UAnimBoneCompressionCodec_ACL>(this, UAnimBoneCompressionCodec_ACL::StaticClass());

		ACLCompressionSettings->Codecs.Add(ACLCodec);
		ACLCompressionSettings->AddToRoot();
	}

	if (TryKeyReduction)
	{
		KeyReductionCompressionSettings = NewObject<UAnimBoneCompressionSettings>(this, UAnimBoneCompressionSettings::StaticClass());
		KeyReductionCodec = NewObject<UAnimCompress_RemoveLinearKeys>(this, UAnimCompress_RemoveLinearKeys::StaticClass());
		KeyReductionCodec->RotationCompressionFormat = ACF_Float96NoW;
		KeyReductionCodec->TranslationCompressionFormat = ACF_None;
		KeyReductionCodec->ScaleCompressionFormat = ACF_None;
		KeyReductionCodec->bActuallyFilterLinearKeys = true;
		KeyReductionCodec->bRetarget = TryKeyReductionRetarget;

		KeyReductionCompressionSettings->Codecs.Add(KeyReductionCodec);
		KeyReductionCompressionSettings->AddToRoot();
	}

	FFileManagerGeneric FileManager;
	FileManager.MakeDirectory(*OutputDir, true);

	if (!HasInput)
	{
		// No source directory, use the current project instead
		ACLRawDir = TEXT("");

		DoActionToAllPackages<UAnimSequence, CompressAnimationsFunctor>(this, Params.ToUpper());
		return 0;
	}
	else
	{
		check(PerformCompression);

		// Use source directory
		ACLRawDir = ParamsMap[TEXT("input")];

#if (ENGINE_MAJOR_VERSION == 4 && ENGINE_MINOR_VERSION >= 26) || ENGINE_MAJOR_VERSION >= 5
		UPackage* TempPackage = CreatePackage(TEXT("/Temp/ACL"));
#else
		UPackage* TempPackage = CreatePackage(nullptr, TEXT("/Temp/ACL"));
#endif

		TArray<FString> FilesLegacy;
		FileManager.FindFiles(FilesLegacy, *ACLRawDir, TEXT(".acl.sjson"));		// Legacy ASCII file format

		TArray<FString> FilesBinary;
		FileManager.FindFiles(FilesBinary, *ACLRawDir, TEXT(".acl"));			// ACL 2.0+ binary format

		TArray<FString> Files;
		Files.Append(FilesLegacy);
		Files.Append(FilesBinary);

		for (const FString& Filename : Files)
		{
			const FString ACLClipPath = FPaths::Combine(*ACLRawDir, *Filename);

			FString UEStatFilename = Filename.Replace(TEXT(".acl.sjson"), TEXT("_stats.sjson"), ESearchCase::CaseSensitive);
			UEStatFilename = UEStatFilename.Replace(TEXT(".acl"), TEXT("_stats.sjson"), ESearchCase::CaseSensitive);

			const FString UEStatPath = FPaths::Combine(*OutputDir, *UEStatFilename);

			if (ResumeTask && FileManager.FileExists(*UEStatPath))
			{
				continue;
			}

			UE_LOGF(LogAnimationCompression, Verbose, "Compressing: %ls", *Filename);

			FArchive* StatWriter = FileManager.CreateFileWriter(*UEStatPath);
			if (StatWriter == nullptr)
			{
				// Opening the file handle can fail if the file path is too long on Windows. UE does not properly handle long paths
				// and adding the \\?\ prefix manually doesn't work, UE mangles it when it normalizes the path.
				continue;
			}

			UESJSONStreamWriter StreamWriter(StatWriter);
			sjson::Writer Writer(StreamWriter);

			acl::track_array_qvvf ACLTracks;

			const TCHAR* ErrorMsg = ReadACLClip(FileManager, ACLClipPath, ACLAllocatorImpl, ACLTracks);
			if (ErrorMsg == nullptr)
			{
				USkeleton* UESkeleton = NewObject<USkeleton>(TempPackage, USkeleton::StaticClass());
				ConvertSkeleton(ACLTracks, UESkeleton);

				UAnimSequence* UEClip = NewObject<UAnimSequence>(TempPackage, UAnimSequence::StaticClass());
				ConvertClip(ACLTracks, UEClip, UESkeleton);

				// Make sure any pending async compression that might have started during load or construction is done
				UEClip->WaitOnExistingCompression();

				FCompressionContext Context;
				Context.AutoCompressor = AutoCompressionSettings;
				Context.ACLCompressor = ACLCompressionSettings;
				Context.KeyReductionCompressor = KeyReductionCompressionSettings;
				Context.UEClip = UEClip;
				Context.UESkeleton = UESkeleton;
				Context.ACLTracks = MoveTemp(ACLTracks);

				Context.ACLRawSize = Context.ACLTracks.get_raw_size();
				Context.UERawSize = UEClip->GetApproxRawSize();

				Writer["duration"] = GetSequenceLength(*UEClip);
				Writer["num_samples"] = Context.ACLTracks.get_num_samples_per_track();
				Writer["ue4_raw_size"] = Context.UERawSize;
				Writer["acl_raw_size"] = Context.ACLRawSize;

				if (TryAutomaticCompression)
				{
					CompressWithUEAuto(Context, PerformExhaustiveDump, Writer);

					ClearCompressedData(UEClip);
				}

				if (TryACLCompression)
				{
					CompressWithACL(Context, PerformExhaustiveDump, Writer);

					ClearCompressedData(UEClip);
				}

				if (TryKeyReduction)
				{
					CompressWithUEKeyReduction(Context, PerformExhaustiveDump, Writer);

					ClearCompressedData(UEClip);
				}

				ClearClip(UEClip);
			}
			else
			{
				Writer["error"] = TCHAR_TO_ANSI(ErrorMsg);
			}

			StatWriter->Close();
		}
	}

	return 0;
}
