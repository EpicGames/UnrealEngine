// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "AnimationCompressionLibraryDatabaseFactory.h"
#include "AnimationCompressionLibraryDatabase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationCompressionLibraryDatabaseFactory)

UAnimationCompressionLibraryDatabaseFactory::UAnimationCompressionLibraryDatabaseFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	SupportedClass = UAnimationCompressionLibraryDatabase::StaticClass();
}

UObject* UAnimationCompressionLibraryDatabaseFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	return NewObject<UAnimationCompressionLibraryDatabase>(InParent, Class, Name, Flags);
}

