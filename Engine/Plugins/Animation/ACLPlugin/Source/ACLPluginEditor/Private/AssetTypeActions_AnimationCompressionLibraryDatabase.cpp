// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "AssetTypeActions_AnimationCompressionLibraryDatabase.h"
#include "AnimBoneCompressionCodec_ACLDatabase.h"

#include "Styling/AppStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

void FAssetTypeActions_AnimationCompressionLibraryDatabase::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
{
	TSharedRef<FSimpleAssetEditor> AssetEditor = FSimpleAssetEditor::CreateEditor(EToolkitMode::Standalone, EditWithinLevelEditor, InObjects);

	auto DatabaseAssets = GetTypedWeakObjectPtrs<UAnimationCompressionLibraryDatabase>(InObjects);
	if (DatabaseAssets.Num() == 1)
	{
		TSharedPtr<class FUICommandList> PluginCommands = MakeShareable(new FUICommandList);
		TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);
		ToolbarExtender->AddToolBarExtension("Asset", EExtensionHook::After, PluginCommands, FToolBarExtensionDelegate::CreateRaw(this, &FAssetTypeActions_AnimationCompressionLibraryDatabase::AddToolbarExtension, DatabaseAssets[0]));
		AssetEditor->AddToolbarExtender(ToolbarExtender);

		AssetEditor->RegenerateMenusAndToolbars();
	}
}

void FAssetTypeActions_AnimationCompressionLibraryDatabase::AddToolbarExtension(FToolBarBuilder& Builder, TWeakObjectPtr<UAnimationCompressionLibraryDatabase> DatabasePtr)
{
	Builder.BeginSection("Build");
	Builder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &FAssetTypeActions_AnimationCompressionLibraryDatabase::ExecuteBuild, DatabasePtr)
		),
		NAME_None,
		FText::FromString(TEXT("Build")),
		FText::FromString(TEXT("Builds the database from all the animation sequences that reference this database through their codec.")),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ApplyCompression")
	);
	Builder.EndSection();
}

void FAssetTypeActions_AnimationCompressionLibraryDatabase::GetActions(const TArray<UObject*>& InObjects, FMenuBuilder& MenuBuilder)
{
	auto DatabaseAssets = GetTypedWeakObjectPtrs<UAnimationCompressionLibraryDatabase>(InObjects);

	if (DatabaseAssets.Num() != 1)
	{
		return;
	}

	MenuBuilder.AddMenuEntry(
		FText::FromString(TEXT("Build")),
		FText::FromString(TEXT("Builds the database from all the animation sequences that reference this database through their codec.")),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Persona.ApplyCompression.Small"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FAssetTypeActions_AnimationCompressionLibraryDatabase::ExecuteBuild, DatabaseAssets[0])
		)
	);
}

void FAssetTypeActions_AnimationCompressionLibraryDatabase::ExecuteBuild(TWeakObjectPtr<UAnimationCompressionLibraryDatabase> DatabasePtr)
{
	if (!DatabasePtr.IsValid())
	{
		return;
	}

	UAnimationCompressionLibraryDatabase* Database = DatabasePtr.Get();
	Database->UpdateReferencingAnimSequenceList();
}

const TArray<FText>& FAssetTypeActions_AnimationCompressionLibraryDatabase::GetSubMenus() const
{
	static const TArray<FText> SubMenus
	{
		NSLOCTEXT("AssetTypeActions_AnimationCompressionLibraryDatabase", "AnimAdvancedSubMenu", "Advanced")
	};
	return SubMenus;
}
