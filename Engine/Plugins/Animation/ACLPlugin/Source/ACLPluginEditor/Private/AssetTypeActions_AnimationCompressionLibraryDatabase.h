// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"
#include "AnimationCompressionLibraryDatabase.h"

class FAssetTypeActions_AnimationCompressionLibraryDatabase : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return FText::FromString(TEXT("ACL Database")); }
	virtual FColor GetTypeColor() const override { return FColor(255, 255, 0); }
	virtual UClass* GetSupportedClass() const override { return UAnimationCompressionLibraryDatabase::StaticClass(); }
	virtual bool CanFilter() override { return true; }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Animation; }
	virtual const TArray<FText>& GetSubMenus() const override;

	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override;
	
	virtual void GetActions(const TArray<UObject*>& InObjects, FMenuBuilder& MenuBuilder) override;

private:
	void AddToolbarExtension(FToolBarBuilder& Builder, TWeakObjectPtr<UAnimationCompressionLibraryDatabase> DatabasePtr);
	void ExecuteBuild(TWeakObjectPtr<UAnimationCompressionLibraryDatabase> DatabasePtr);
};
