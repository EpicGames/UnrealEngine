// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// Copyright 2020 Nicholas Frechette. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/** The editor ACL plugin module interface. */
class IACLPluginEditor : public IModuleInterface
{
public:
	static inline IACLPluginEditor& Get()
	{
		return FModuleManager::LoadModuleChecked<IACLPluginEditor>("ACLPluginEditor");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("ACLPluginEditor");
	}
};
