<?xml version="1.0" encoding="utf-8"?>

<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Type Name="sjson::StringView">
    <DisplayString>{m_c_str,[m_length]s8}</DisplayString>
  </Type>

</AutoVisualizer>
